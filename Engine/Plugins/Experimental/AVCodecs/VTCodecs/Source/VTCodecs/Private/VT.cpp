// Copyright Epic Games, Inc. All Rights Reserved.

#include "VT.h"

#include "AVResult.h"
#include "HAL/PlatformProcess.h"
#include "HAL/Thread.h"
#include "RHI.h"

THIRD_PARTY_INCLUDES_START
#if PLATFORM_MAC
	// IOKit pull mac only includes
	#include <IOKit/IOKitLib.h>
#endif
THIRD_PARTY_INCLUDES_END

REGISTER_TYPEID(FVT);

namespace
{
	// If IoSurface is unsupported, then VTIsHardwareDecodeSupported will assert internally.	
	bool IsIoSurfaceAvailable()
	{
#if PLATFORM_MAC
		// IOSurfaceRoot is determined by running `ioreg -n IOSurfaceRoot` which prints
		// "CFBundleIdentifier" = "com.apple.iokit.IOSurface"
		// "IOMatchCategory" = "IOSurfaceRoot"
		io_service_t Service = IOServiceGetMatchingService(kIOMainPortDefault, IOServiceMatching("IOSurfaceRoot"));
		if (!Service)
		{
			FAVResult::Log(EAVResult::Warning, TEXT("Failed to get IOSurfaceRoot service"));
			return false;
		}
		io_connect_t Connect = IO_OBJECT_NULL;
		kern_return_t KernResult = IOServiceOpen(Service, mach_task_self(), 0, &Connect);
		IOObjectRelease(Service);
		if (KernResult != KERN_SUCCESS)
		{
			FAVResult::Log(EAVResult::Warning, FString::Printf(TEXT("IOServiceOpen Failed to open IOSurfaceRoot with: [%d]"), static_cast<int>(KernResult)));
			return false;
		}
		IOServiceClose(Connect);
#endif
		return true;
	}
}

FVT::FVT() : bIsIoSurfaceAvailable(IsIoSurfaceAvailable())
{
}

bool FVT::IsValid() const
{
	return bHasCompatibleGPU && bIsIoSurfaceAvailable;
}
