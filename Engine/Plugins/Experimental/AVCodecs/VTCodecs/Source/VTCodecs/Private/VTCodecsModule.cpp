// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"
#include "Misc/App.h"

#include "Video/Encoders/VideoEncoderVT.h"
#include "Video/Encoders/Configs/VideoEncoderConfigH264.h"
#include "Video/Encoders/Configs/VideoEncoderConfigH265.h"

#include "Video/Decoders/VideoDecoderVT.h"
#include "Video/Decoders/Configs/VideoDecoderConfigVP9.h"
#include "Video/Decoders/Configs/VideoDecoderConfigH264.h"
#include "Video/Decoders/Configs/VideoDecoderConfigH265.h"

#include "Video/Resources/Metal/VideoResourceMetal.h"

THIRD_PARTY_INCLUDES_START
#include <VideoToolbox/VideoToolbox.h>
THIRD_PARTY_INCLUDES_END

class FVTModule : public IModuleInterface
{
public:
	virtual void StartupModule() override
	{       
        TFunction<bool(TSharedRef<FAVDevice> const&, const TCHAR*)> CheckCodecSupport = [](TSharedRef<FAVDevice> const& Device, const TCHAR* Codec) {
            CFArrayRef EncoderList = NULL;
            bool bSupportsHardware = false;

            OSStatus Status = VTCopyVideoEncoderList(NULL, &EncoderList);
            if (Status != 0) 
            {
                return false;
            }

			CFStringRef TargetCodec = CFStringCreateWithCString(kCFAllocatorDefault, TCHAR_TO_ANSI(Codec), kCFStringEncodingUTF8);
			if (!TargetCodec)
			{
				if (EncoderList) 
				{
					CFRelease(EncoderList);
				}
				return false;
			}
			
            CFIndex EncoderCount = CFArrayGetCount(EncoderList);
            for (CFIndex i = 0; i < EncoderCount; i++) 
            {
				CFDictionaryRef EncoderDict = (CFDictionaryRef)CFArrayGetValueAtIndex(EncoderList, i);
                if (EncoderDict)
                {
					CFStringRef EncoderCodec = (CFStringRef)CFDictionaryGetValue(EncoderDict, kVTVideoEncoderList_CodecName);
					if (!EncoderCodec)
					{
						continue;
					}

					CFComparisonResult CompareResult = CFStringCompare(EncoderCodec, TargetCodec, 0);
		 			if (CompareResult != kCFCompareEqualTo)
					{
						continue;
					}
					
                    CFBooleanRef bIsHardware = (CFBooleanRef)CFDictionaryGetValue(EncoderDict, kVTVideoEncoderList_IsHardwareAccelerated);
					bSupportsHardware |= bIsHardware ? (bool)CFBooleanGetValue(bIsHardware) : false;
                }
            }

            if (EncoderList) 
            {
                CFRelease(EncoderList);
            }
			
			if (TargetCodec)
			{
				CFRelease(TargetCodec);
			}

			return bSupportsHardware;
		};

        FVideoEncoder::RegisterPermutationsOf<TVideoEncoderVT<FVideoResourceMetal>>
            ::With<FVideoResourceMetal>
            ::And<FVideoEncoderConfigVT, FVideoEncoderConfigH264>(
                [CheckCodecSupport](TSharedRef<FAVDevice> const& NewDevice, TSharedRef<FAVInstance> const& NewInstance)
                {
					static bool bSupportsCodec = FAPI::Get<FVT>().IsValid() && (CheckCodecSupport(NewDevice, TEXT("H.264")) || CheckCodecSupport(NewDevice, TEXT("avc1")));

					return bSupportsCodec;
                });

        FVideoEncoder::RegisterPermutationsOf<TVideoEncoderVT<FVideoResourceMetal>>
            ::With<FVideoResourceMetal>
            ::And<FVideoEncoderConfigVT, FVideoEncoderConfigH265>(
                [CheckCodecSupport](TSharedRef<FAVDevice> const& NewDevice, TSharedRef<FAVInstance> const& NewInstance)
                {
					static bool bSupportsCodec = FAPI::Get<FVT>().IsValid() && CheckCodecSupport(NewDevice, TEXT("HEVC"));

					return bSupportsCodec;
                });

#if PLATFORM_MAC
		// TODO(Eden.Harris) VTRegisterSupplementalVideoDecoderIfAvailable is only available on mac
        VTRegisterSupplementalVideoDecoderIfAvailable(kCMVideoCodecType_VP9);
#endif

		FVideoDecoder::RegisterPermutationsOf<FVideoDecoderVT>
			::With<FVideoResourceMetal>
			::And<FVideoDecoderConfigVT, FVideoDecoderConfigH264>(
				[](TSharedRef<FAVDevice> const& NewDevice, TSharedRef<FAVInstance> const& NewInstance)
				{
					// IsValid must be called before VTIsHardwareDecodeSupported as it needs IOSurface support
					static bool bSupportsCodec = FAPI::Get<FVT>().IsValid() && VTIsHardwareDecodeSupported(kCMVideoCodecType_H264);
					
					return bSupportsCodec;
				});

		FVideoDecoder::RegisterPermutationsOf<FVideoDecoderVT>
			::With<FVideoResourceMetal>
			::And<FVideoDecoderConfigVT, FVideoDecoderConfigH265>(
				[](TSharedRef<FAVDevice> const& NewDevice, TSharedRef<FAVInstance> const& NewInstance)
				{
					// IsValid must be called before VTIsHardwareDecodeSupported as it needs IOSurface support
					static bool bSupportsCodec = FAPI::Get<FVT>().IsValid() && VTIsHardwareDecodeSupported(kCMVideoCodecType_HEVC);
					
					return bSupportsCodec;
				});

		FVideoDecoder::RegisterPermutationsOf<FVideoDecoderVT>
			::With<FVideoResourceMetal>
			::And<FVideoDecoderConfigVT, FVideoDecoderConfigVP9>(
				[](TSharedRef<FAVDevice> const& NewDevice, TSharedRef<FAVInstance> const& NewInstance)
				{
					// IsValid must be called before VTIsHardwareDecodeSupported as it needs IOSurface support
					static bool bSupportsCodec = FAPI::Get<FVT>().IsValid() && VTIsHardwareDecodeSupported(kCMVideoCodecType_VP9);
					
					return bSupportsCodec;
				});

	}
};

IMPLEMENT_MODULE(FVTModule, VTCodecs);
