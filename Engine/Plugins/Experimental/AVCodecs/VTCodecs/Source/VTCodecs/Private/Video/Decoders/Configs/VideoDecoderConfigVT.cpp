// Copyright Epic Games, Inc. All Rights Reserved.

#include "Video/Decoders/Configs/VideoDecoderConfigVT.h"


#include "Video/Decoders/Configs/VideoDecoderConfigH264.h"
#include "Video/Decoders/Configs/VideoDecoderConfigH265.h"
#include "Video/Decoders/Configs/VideoDecoderConfigVP9.h"

REGISTER_TYPEID(FVideoDecoderConfigVT);

TAVResult<OSType> FVideoDecoderConfigVT::ConvertFormat(EVideoFormat const& Format)
{
	switch (Format)
	{
		case EVideoFormat::BGRA:
			return kCVPixelFormatType_32RGBA;
		case EVideoFormat::ABGR10:
			return kCVPixelFormatType_ARGB2101010LEPacked;
		case EVideoFormat::NV12:
			return kCVPixelFormatType_420YpCbCr8BiPlanarFullRange;
		default:
			return FAVResult(EAVResult::ErrorUnsupported, FString::Printf(TEXT("Pixel format %d is not supported"), EnumToUnderlyingType(Format)), TEXT("NVENC"));
	}
}


template<>
DLLEXPORT FAVResult FAVExtension::TransformConfig(FVideoDecoderConfigVT& OutConfig, FVideoDecoderConfig const& InConfig)
{
	return EAVResult::Success;
}

template <>
DLLEXPORT FAVResult FAVExtension::TransformConfig(FVideoDecoderConfig& OutConfig, FVideoDecoderConfigVT const& InConfig)
{
	return EAVResult::Success;
}

template <>
DLLEXPORT FAVResult FAVExtension::TransformConfig(FVideoDecoderConfigVT& OutConfig, FVideoDecoderConfigH264 const& InConfig)
{
	OutConfig.Codec = kCMVideoCodecType_H264;

	return FAVExtension::TransformConfig<FVideoDecoderConfigVT, FVideoDecoderConfig>(OutConfig, InConfig);
}

template <>
DLLEXPORT FAVResult FAVExtension::TransformConfig(FVideoDecoderConfigVT& OutConfig, FVideoDecoderConfigH265 const& InConfig)
{
	OutConfig.Codec = kCMVideoCodecType_HEVC;

	return FAVExtension::TransformConfig<FVideoDecoderConfigVT, FVideoDecoderConfig>(OutConfig, InConfig);
}

template <>
DLLEXPORT FAVResult FAVExtension::TransformConfig(FVideoDecoderConfigVT& OutConfig, FVideoDecoderConfigVP9 const& InConfig)
{
	OutConfig.Codec = kCMVideoCodecType_VP9;

	return FAVExtension::TransformConfig<FVideoDecoderConfigVT, FVideoDecoderConfig>(OutConfig, InConfig);
}
