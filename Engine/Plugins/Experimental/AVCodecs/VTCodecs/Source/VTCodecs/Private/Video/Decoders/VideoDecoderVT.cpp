// Copyright Epic Games, Inc. All Rights Reserved.

#include "Video/Decoders/VideoDecoderVT.h"
#include "Video/Resources/Metal/VideoResourceMetal.h"

#include "Video/Util/NaluRewriter.h"

#include "DynamicRHI.h"

THIRD_PARTY_INCLUDES_START
#include <VideoToolbox/VideoToolbox.h>
#include <CoreMedia/CMSync.h>
THIRD_PARTY_INCLUDES_END

#define CONDITIONAL_RELEASE(x)          \
	if (x)                              \
	{                                   \
		CFRelease(x);                   \
		x = nullptr;                    \
	}


namespace Internal
{
    void VTDecompressionOutputCallback(void* Decoder, void* Params, OSStatus Status, VTDecodeInfoFlags InfoFlags, CVImageBufferRef ImageBuffer, CMTime Timestamp, CMTime Duration)
    {
        static_cast<FVideoDecoderVT*>(Decoder)->HandleFrame(Params, Status, InfoFlags, ImageBuffer, Timestamp, Duration);
    }
}

FVideoDecoderVT::~FVideoDecoderVT()
{
	Close();
}

bool FVideoDecoderVT::IsOpen() const
{
	return bIsOpen;
}

FAVResult FVideoDecoderVT::Open(TSharedRef<FAVDevice> const& NewDevice, TSharedRef<FAVInstance> const& NewInstance)
{
	Close();

	TVideoDecoder<FVideoResourceMetal, FVideoDecoderConfigVT>::Open(NewDevice, NewInstance);

	MemoryPool = CMMemoryPoolCreate(nullptr);

	FrameCount = 0;

	bIsOpen = true;

	return EAVResult::Success;
}

void FVideoDecoderVT::Close()
{
	DestroyDecompressionSession();

	if(MemoryPool)
	{
		CMMemoryPoolInvalidate(MemoryPool);
		CFRelease(MemoryPool);
	}

	bIsOpen = false;
}

void FVideoDecoderVT::DestroyDecompressionSession()
{
	if (Decoder) 
	{
		VTDecompressionSessionInvalidate(Decoder);	
		CFRelease(Decoder);
		Decoder = nullptr;
	}
}

void FVideoDecoderVT::ConfigureDecompressionSession()
{
	VTSessionSetProperty(Decoder, kVTDecompressionPropertyKey_RealTime, kCFBooleanTrue);
}

bool FVideoDecoderVT::IsInitialized() const
{
	return Decoder != nullptr;
}

FAVResult FVideoDecoderVT::ApplyConfig()
{
	if (IsOpen())
	{
		FVideoDecoderConfigVT const& PendingConfig = this->GetPendingConfig();
		if (this->AppliedConfig != PendingConfig)
		{
			if (IsInitialized())
			{
				// VideoToolbox decoder doesn't support reconfiguration. If any aspect of the config changes,
				// the entire session must be re-created
				if (Decoder)
				{
					DestroyDecompressionSession();
					FAVResult::Log(EAVResult::Success, TEXT("Re-initializing decoding session"), TEXT("VT"));
				}
			}

			if (!IsInitialized())
			{
				if (!PendingConfig.VideoFormat)
				{
					return FAVResult(EAVResult::PendingInput);
				}

				// Set source image buffer attributes. These attributes will be present on
				// buffers retrieved from the decoder's pixel buffer pool.
				CFMutableDictionaryRef SourceAttributes = CFDictionaryCreateMutable(kCFAllocatorDefault, 3, &kCFTypeDictionaryKeyCallBacks, &kCFTypeDictionaryValueCallBacks);
				CFDictionarySetValue(SourceAttributes, kCVPixelBufferOpenGLCompatibilityKey, kCFBooleanTrue);
				CFDictionaryRef IOSurfaceValue = CFDictionaryCreate(kCFAllocatorDefault, nullptr, nullptr, 0, &kCFTypeDictionaryKeyCallBacks, &kCFTypeDictionaryValueCallBacks);
				CFDictionarySetValue(SourceAttributes, kCVPixelBufferIOSurfacePropertiesKey, IOSurfaceValue);
				// TODO (belchy06): This should support more than ARGB8 pixel format
				int64 PixelType = kCVPixelFormatType_32BGRA;
				CFNumberRef PixelFormat = CFNumberCreate(nullptr, kCFNumberLongType, &PixelType);
				CFDictionarySetValue(SourceAttributes, kCVPixelBufferPixelFormatTypeKey, PixelFormat);

				CONDITIONAL_RELEASE(IOSurfaceValue);
				CONDITIONAL_RELEASE(PixelFormat);

				VTDecompressionOutputCallbackRecord Record = 
				{
					Internal::VTDecompressionOutputCallback, 
					this
				};

				OSStatus Result = VTDecompressionSessionCreate(kCFAllocatorDefault, PendingConfig.VideoFormat, nullptr, SourceAttributes, &Record, &Decoder);

				CONDITIONAL_RELEASE(SourceAttributes);

				if(Result != 0)
				{
					DestroyDecompressionSession();
					return FAVResult(EAVResult::ErrorCreating, TEXT("Failed to create VTDecompressionSession"), TEXT("VT"), Result);
				}

				ConfigureDecompressionSession();
			}
		}

		return TVideoDecoder<FVideoResourceMetal, FVideoDecoderConfigVT>::ApplyConfig();
	}

	return FAVResult(EAVResult::ErrorInvalidState, TEXT("Decoder not open"), TEXT("VT"));
}

FAVResult FVideoDecoderVT::SendPacket(FVideoPacket const& Packet)
{
	if (IsOpen())
	{
		// We've received a call to decode a frame, we can now parse the information from
		// the bitstream, configure our config and initialize the session
		FVideoDecoderConfigVT& PendingConfig = this->EditPendingConfig();

		CMVideoFormatDescriptionRef InputFormat = nullptr;
		if(PendingConfig.Codec == kCMVideoCodecType_H264)
		{
			InputFormat = NaluRewriter::CreateH264VideoFormatDescription(Packet.DataPtr.Get(), Packet.DataSize);
		}
		else if(PendingConfig.Codec == kCMVideoCodecType_HEVC)
		{
			InputFormat = NaluRewriter::CreateH265VideoFormatDescription(Packet.DataPtr.Get(), Packet.DataSize);
		}
		else if (PendingConfig.Codec == kCMVideoCodecType_VP9)
		{
			InputFormat = NaluRewriter::CreateVP9VideoFormatDescription(Packet.DataPtr.Get(), Packet.DataSize);
		}
		else
		{
			return FAVResult(EAVResult::Error, TEXT("Unsupported codec"), TEXT("VT"));
		}

		if (InputFormat && (!this->AppliedConfig.VideoFormat || !CMFormatDescriptionEqual(InputFormat, this->AppliedConfig.VideoFormat)))
		{
			PendingConfig.SetVideoFormat(InputFormat);
		}
		
		FAVResult AVResult = ApplyConfig();
		CONDITIONAL_RELEASE(InputFormat);
		if (AVResult.IsNotSuccess())
		{
			return AVResult;
		}

		if(!this->AppliedConfig.VideoFormat)
		{
			return FAVResult(EAVResult::WarningInvalidState, TEXT("Missing video format. Frame with sps/pps required."), TEXT("VT"));
		}

		CMSampleBufferRef SampleBuffer = nullptr;
		if(this->AppliedConfig.Codec == kCMVideoCodecType_H264)
		{
			if(!NaluRewriter::H264AnnexBBufferToCMSampleBuffer(Packet.DataPtr.Get(), Packet.DataSize, this->AppliedConfig.VideoFormat, &SampleBuffer, MemoryPool))
			{
				return FAVResult(EAVResult::Error, TEXT("Failed to get SampleBuffer"), TEXT("VT"));
			}
		}
		else if(this->AppliedConfig.Codec == kCMVideoCodecType_HEVC)
		{
			if(!NaluRewriter::H265AnnexBBufferToCMSampleBuffer(Packet.DataPtr.Get(), Packet.DataSize, this->AppliedConfig.VideoFormat, &SampleBuffer, MemoryPool))
			{
				return FAVResult(EAVResult::Error, TEXT("Failed to get SampleBuffer"), TEXT("VT"));
			}
		}
		else if (this->AppliedConfig.Codec == kCMVideoCodecType_VP9)
		{
			if(!NaluRewriter::VP9BufferToCMSampleBuffer(Packet.DataPtr.Get(), Packet.DataSize, this->AppliedConfig.VideoFormat, &SampleBuffer, MemoryPool))
			{
				return FAVResult(EAVResult::Error, TEXT("Failed to get SampleBuffer"), TEXT("VT"));
			}
		}
		else
		{
			return FAVResult(EAVResult::Error, TEXT("Unsupported codec"), TEXT("VT"));
		}
		
		if(SampleBuffer == nullptr)
		{
			return FAVResult(EAVResult::Error, TEXT("SampleBuffer is nullptr"), TEXT("VT"));
		}

		OSStatus Result = VTDecompressionSessionDecodeFrame(Decoder, SampleBuffer, 0, nullptr, nullptr);

		CFRelease(SampleBuffer);

		if(Result != 0)
		{
			return FAVResult(EAVResult::Error, TEXT("Failed to decode frame"), TEXT("VT"), Result);
		}

		return EAVResult::Success;
	}

	return FAVResult(EAVResult::ErrorInvalidState, TEXT("Decoder not open"), TEXT("VT"));
}

FAVResult FVideoDecoderVT::ReceiveFrame(TResolvableVideoResource<FVideoResourceMetal>& InOutResource)
{
	if(IsOpen())
	{
		if(Frames.Peek())
		{
			TSharedPtr<FFrame> Frame = *Frames.Peek();
			Frames.Pop();

			size_t Width = CVPixelBufferGetWidth(Frame->ImageBuffer);
			size_t Height = CVPixelBufferGetHeight(Frame->ImageBuffer);		

			if (!InOutResource.Resolve(this->GetDevice(), FVideoDescriptor(EVideoFormat::BGRA, Width, Height)))
			{
				return FAVResult(EAVResult::ErrorResolving, TEXT("Failed to resolve frame resource"), TEXT("VT"));
			}

			return InOutResource->CopyFrom(Frame->ImageBuffer);
		}

		return EAVResult::PendingInput;
	}

	return FAVResult(EAVResult::ErrorInvalidState, TEXT("Decoder not open"), TEXT("VT"));
}

FAVResult FVideoDecoderVT::HandleFrame(void* Params, OSStatus Status, VTDecodeInfoFlags InfoFlags, CVImageBufferRef ImageBuffer, CMTime Timestamp, CMTime Duration)
{
	if(IsOpen())
	{
		if(Status != 0)
		{
			return FAVResult(EAVResult::Error, TEXT("Failed to decode"), TEXT("VT"), Status);
		}

		if(!ImageBuffer)
		{
			return FAVResult(EAVResult::Error, TEXT("No output image buffer"), TEXT("VT"), Status);
		}
		
		// The destructor for FFrame releases the ImageBuffer so we need to make sure it's not release until after it's been used
		TSharedPtr<FFrame> Frame = MakeShareable(new FFrame(ImageBuffer, Timestamp, Duration));
		Frames.Enqueue(Frame);

		return EAVResult::Success;
	}
	
	return FAVResult(EAVResult::ErrorInvalidState, TEXT("Decoder not open"), TEXT("VT"));
}
