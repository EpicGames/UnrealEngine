// Copyright Epic Games, Inc. All Rights Reserved.

#include "Video/Encoders/Configs/VideoEncoderConfigVT.h"

#include "Video/Encoders/Configs/VideoEncoderConfigH264.h"
#include "Video/Encoders/Configs/VideoEncoderConfigH265.h"

REGISTER_TYPEID(FVideoEncoderConfigVT);

static uint32 DEFAULT_BITRATE_TARGET = 1000000;
static uint32 DEFAULT_BITRATE_MAX = 10000000;

TAVResult<OSType> FVideoEncoderConfigVT::ConvertFormat(EVideoFormat const& Format)
{
	switch (Format)
	{
	case EVideoFormat::BGRA:
		return kCVPixelFormatType_32RGBA;
	case EVideoFormat::ABGR10:
		return kCVPixelFormatType_ARGB2101010LEPacked;
	case EVideoFormat::NV12:
		return kCVPixelFormatType_420YpCbCr8BiPlanarFullRange;
	default:
		return FAVResult(EAVResult::ErrorUnsupported, FString::Printf(TEXT("Pixel format %d is not supported"), EnumToUnderlyingType(Format)), TEXT("VT"));
	}
}

template <>
DLLEXPORT FAVResult FAVExtension::TransformConfig(FVideoEncoderConfigVT& OutConfig, FVideoEncoderConfig const& InConfig)
{
	OutConfig.Width = InConfig.Width;
	OutConfig.Height = InConfig.Height;
	OutConfig.Preset = InConfig.Preset;
	OutConfig.FrameRate = InConfig.TargetFramerate;
	OutConfig.TargetBitrate = InConfig.TargetBitrate > -1 ? InConfig.TargetBitrate : DEFAULT_BITRATE_TARGET;
	OutConfig.MaxBitrate = InConfig.MaxBitrate > -1 ? InConfig.MaxBitrate : DEFAULT_BITRATE_MAX;
	OutConfig.RateControlMode = InConfig.RateControlMode;
	OutConfig.MinQuality = InConfig.MinQuality;
	OutConfig.MaxQuality = InConfig.MaxQuality;
	OutConfig.KeyframeInterval = InConfig.KeyframeInterval > 0 ? InConfig.KeyframeInterval : 0;

	return EAVResult::Success;
}

template <>
DLLEXPORT FAVResult FAVExtension::TransformConfig(FVideoEncoderConfig& OutConfig, FVideoEncoderConfigVT const& InConfig)
{
	OutConfig.Width = InConfig.Width;
	OutConfig.Height = InConfig.Height;
	OutConfig.Preset = InConfig.Preset;
	OutConfig.TargetFramerate = InConfig.FrameRate;
	OutConfig.TargetBitrate = InConfig.TargetBitrate > -1 ? InConfig.TargetBitrate : DEFAULT_BITRATE_TARGET;
	OutConfig.MaxBitrate = InConfig.MaxBitrate > -1 ? InConfig.MaxBitrate : DEFAULT_BITRATE_MAX;
	OutConfig.RateControlMode = InConfig.RateControlMode;
	OutConfig.MinQuality = InConfig.MinQuality;
	OutConfig.MaxQuality = InConfig.MaxQuality;
	OutConfig.KeyframeInterval = InConfig.KeyframeInterval > 0 ? InConfig.KeyframeInterval : 0;

	return EAVResult::Success;
}

template <>
DLLEXPORT FAVResult FAVExtension::TransformConfig(FVideoEncoderConfigVT& OutConfig, FVideoEncoderConfigH264 const& InConfig)
{
	OutConfig.Codec = kCMVideoCodecType_H264;

	static auto const ConvertEntropyCodingMode = [](EH264EntropyCodingMode EntropyCodingMode) -> TAVResult<CFStringRef>
		{
			switch (EntropyCodingMode)
			{
			case EH264EntropyCodingMode::Auto:
				return nullptr;
			case EH264EntropyCodingMode::CABAC:
				return kVTH264EntropyMode_CABAC;
			case EH264EntropyCodingMode::CAVLC:
				return kVTH264EntropyMode_CAVLC;
			default:
				return FAVResult(EAVResult::ErrorUnsupported, FString::Printf(TEXT("H264 entropy coding mode %d is not supported"), EnumToUnderlyingType(EntropyCodingMode)), TEXT("VT"));
			}
		};

	TAVResult<CFStringRef> const ConvertedEntropyCodingMode = ConvertEntropyCodingMode(InConfig.EntropyCodingMode);
	if (ConvertedEntropyCodingMode.IsNotSuccess())
	{
		return ConvertedEntropyCodingMode;
	}

	OutConfig.EntropyCodingMode = ConvertedEntropyCodingMode;

	static auto const ConvertProfile = [](EH264Profile Profile) -> TAVResult<CFStringRef>
		{
			switch (Profile)
			{
			case EH264Profile::Baseline:
				return kVTProfileLevel_H264_Baseline_AutoLevel;
			case EH264Profile::Main:
				return kVTProfileLevel_H264_Main_AutoLevel;
			case EH264Profile::High:
				return kVTProfileLevel_H264_High_AutoLevel;
			case EH264Profile::ConstrainedHigh:
				return kVTProfileLevel_H264_ConstrainedHigh_AutoLevel;
			default:
				return FAVResult(EAVResult::ErrorUnsupported, FString::Printf(TEXT("H264 profile %d is not supported"), Profile), TEXT("VT"));
			}
		};

	TAVResult<CFStringRef> const ConvertedProfile = ConvertProfile(InConfig.Profile);
	if (ConvertedProfile.IsNotSuccess())
	{
		return ConvertedProfile;
	}

	OutConfig.Profile = ConvertedProfile.ReturnValue;

	return FAVExtension::TransformConfig<FVideoEncoderConfigVT, FVideoEncoderConfig>(OutConfig, InConfig);
}

template <>
DLLEXPORT FAVResult FAVExtension::TransformConfig(FVideoEncoderConfigVT& OutConfig, FVideoEncoderConfigH265 const& InConfig)
{
	static auto const ConvertProfile = [](EH265Profile Profile) -> TAVResult<CFStringRef>
		{
			switch (Profile)
			{
			case EH265Profile::Main:
				return kVTProfileLevel_HEVC_Main_AutoLevel;
			case EH265Profile::Main10:
				return kVTProfileLevel_HEVC_Main10_AutoLevel;
			case EH265Profile::Main422_10:
				return kVTProfileLevel_HEVC_Main42210_AutoLevel;
			default:
				return FAVResult(EAVResult::ErrorUnsupported, FString::Printf(TEXT("H265 profile %d is not supported"), Profile), TEXT("VT"));
			}
		};

	OutConfig.Codec = kCMVideoCodecType_HEVC;

	TAVResult<CFStringRef> const ConvertedProfile = ConvertProfile(InConfig.Profile);
	if (ConvertedProfile.IsNotSuccess())
	{
		return ConvertedProfile;
	}

	OutConfig.Profile = ConvertedProfile.ReturnValue;

	return FAVExtension::TransformConfig<FVideoEncoderConfigVT, FVideoEncoderConfig>(OutConfig, InConfig);
}
