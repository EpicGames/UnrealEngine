// Copyright Epic Games, Inc. All Rights Reserved.

#include "Video/Encoders/VideoEncoderVT.h"
#include "Video/Resources/Metal/VideoResourceMetal.h"

namespace Internal
{
    void VTCompressionOutputCallback(void* Encoder, void* Params, OSStatus Status, VTEncodeInfoFlags InfoFlags, CMSampleBufferRef SampleBuffer) 
    {
        static_cast<TVideoEncoderVT<FVideoResourceMetal>*>(Encoder)->HandlePacket(Params, Status, InfoFlags, SampleBuffer);
    }
}
