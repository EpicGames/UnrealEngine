// Copyright Epic Games, Inc. All Rights Reserved.

#include "Video/Util/NaluRewriter.h"

#include "AVResult.h"

#include "Video/Decoders/Configs/VideoDecoderConfigVP9.h"

THIRD_PARTY_INCLUDES_START
#include <CoreFoundation/CoreFoundation.h>
THIRD_PARTY_INCLUDES_END

#include <memory>

#define CONDITIONAL_RELEASE(x)          \
    if (x)                              \
    {                                   \
        CFRelease(x);                   \
        x = nullptr;                    \
    }

const uint8 kAnnexBHeaderBytes[4] = {0, 0, 0, 1};
const size_t kAvccHeaderByteSize = sizeof(uint32_t);

bool NaluRewriter::H264CMSampleBufferToAnnexBBuffer(CMSampleBufferRef avcc_sample_buffer,bool is_keyframe, TArray<uint8>& annexb_buffer) 
{
    check(avcc_sample_buffer);

    // Get format description from the sample buffer.
    CMVideoFormatDescriptionRef description = CMSampleBufferGetFormatDescription(avcc_sample_buffer);
    if (description == nullptr) 
    {
        FAVResult::Log(EAVResult::Error, TEXT("Failed to get sample buffer's description"), TEXT("VT"));
        return false;
    }

    // Get parameter set information.
    int nalu_header_size = 0;
    size_t param_set_count = 0;
    OSStatus status = CMVideoFormatDescriptionGetH264ParameterSetAtIndex(description, 0, nullptr, nullptr, &param_set_count, &nalu_header_size);
    if (status != 0) 
    {
        FAVResult::Log(EAVResult::Error, TEXT("Failed to get parameter set"), TEXT("VT"));
        return false;
    }
    check(nalu_header_size == kAvccHeaderByteSize);
    check(param_set_count == 2);

    // Truncate any previous data in the buffer without changing its capacity.
    annexb_buffer.SetNum(0, EAllowShrinking::No);

    // Place all parameter sets at the front of buffer.
    if (is_keyframe) 
    {
        size_t param_set_size = 0;
        const uint8* param_set = nullptr;
        for (size_t i = 0; i < param_set_count; ++i) 
        {
            status = CMVideoFormatDescriptionGetH264ParameterSetAtIndex(description, i, &param_set, &param_set_size, nullptr, nullptr);
            if (status != 0) 
            {
                FAVResult::Log(EAVResult::Error, TEXT("Failed to get parameter set"), TEXT("VT"));
                return false;
            }

            // Update buffer.
            annexb_buffer.Append(kAnnexBHeaderBytes, sizeof(kAnnexBHeaderBytes));
            annexb_buffer.Append(reinterpret_cast<const uint8*>(param_set), param_set_size);
        }
    }

    // Get block buffer from the sample buffer.
    CMBlockBufferRef block_buffer = CMSampleBufferGetDataBuffer(avcc_sample_buffer);
    if (block_buffer == nullptr) 
    {
        FAVResult::Log(EAVResult::Error, TEXT("Failed to get sample buffer's block buffer"), TEXT("VT"));
        return false;
    }
    
    CMBlockBufferRef contiguous_buffer = nullptr;
    // Make sure block buffer is contiguous.
    if (!CMBlockBufferIsRangeContiguous(block_buffer, 0, 0)) 
    {
        status = CMBlockBufferCreateContiguous(nullptr, block_buffer, nullptr, nullptr, 0, 0, 0, &contiguous_buffer);
        if (status != 0) 
        {
            FAVResult::Log(EAVResult::Error, TEXT("Failed to flatten non-contiguous block buffer"), TEXT("VT"));
            return false;
        }
    } 
    else 
    {
        contiguous_buffer = block_buffer;
        // Retain to make cleanup easier.
        CFRetain(contiguous_buffer);
        block_buffer = nullptr;
    }

    // Now copy the actual data.
    char* data_ptr = nullptr;
    size_t block_buffer_size = CMBlockBufferGetDataLength(contiguous_buffer);
    status = CMBlockBufferGetDataPointer(contiguous_buffer, 0, nullptr, nullptr, &data_ptr);
    if (status != 0) 
    {
        FAVResult::Log(EAVResult::Error, TEXT("Failed to get block buffer data"), TEXT("VT"));
        CFRelease(contiguous_buffer);
        return false;
    }
    size_t bytes_remaining = block_buffer_size;
    while (bytes_remaining > 0) 
    {
        // The size type here must match |nalu_header_size|, we expect 4 bytes.
        // Read the length of the next packet of data. Must convert from big endian
        // to host endian.
        check(bytes_remaining > (size_t)nalu_header_size);
        uint32_t* uint32_data_ptr = reinterpret_cast<uint32_t*>(data_ptr);
        uint32_t packet_size = CFSwapInt32BigToHost(*uint32_data_ptr);
        // Update buffer.
        annexb_buffer.Append(kAnnexBHeaderBytes, sizeof(kAnnexBHeaderBytes));
        annexb_buffer.Append(reinterpret_cast<uint8*>(data_ptr + (size_t)nalu_header_size), packet_size);

        size_t bytes_written = packet_size + sizeof(kAnnexBHeaderBytes);
        bytes_remaining -= bytes_written;
        data_ptr += bytes_written;
    }

    check(bytes_remaining == (size_t)0);

  CFRelease(contiguous_buffer);
  return true;
}

bool NaluRewriter::H264AnnexBBufferToCMSampleBuffer(const uint8* annexb_buffer, size_t annexb_buffer_size, CMVideoFormatDescriptionRef video_format, CMSampleBufferRef* out_sample_buffer, CMMemoryPoolRef memory_pool) 
{
    check(annexb_buffer);
    check(out_sample_buffer);
    check(video_format);
    *out_sample_buffer = nullptr;

    AnnexBBufferReader reader(annexb_buffer, annexb_buffer_size);
    if (reader.SeekToNextNaluOfType(H264::ENaluType::SequenceParameterSet)) 
    {
        // Buffer contains an SPS NALU - skip it and the following PPS
        const uint8* data;
        size_t data_len;
        if (!reader.ReadNalu(&data, &data_len)) 
        {
            FAVResult::Log(EAVResult::Error, TEXT("Failed to read SPS"), TEXT("VT"));
            return false;
        }
        if (!reader.ReadNalu(&data, &data_len)) 
        {
            FAVResult::Log(EAVResult::Error, TEXT("Failed to read PPS"), TEXT("VT"));
            return false;
        }
    }
    else 
    {
        // No SPS NALU - start reading from the first NALU in the buffer
        reader.SeekToStart();
    }

    // Allocate memory as a block buffer.
    CMBlockBufferRef block_buffer = nullptr;
    CFAllocatorRef block_allocator = CMMemoryPoolGetAllocator(memory_pool);
    OSStatus status = CMBlockBufferCreateWithMemoryBlock(kCFAllocatorDefault, nullptr, reader.BytesRemainingForAVC(), block_allocator, nullptr, 0, reader.BytesRemainingForAVC(), kCMBlockBufferAssureMemoryNowFlag, &block_buffer);
    if (status != kCMBlockBufferNoErr) 
    {
        FAVResult::Log(EAVResult::Error, TEXT("Failed to allocate block buffer"), TEXT("VT"));
        return false;
    }

    // Make sure block buffer is contiguous.
    CMBlockBufferRef contiguous_buffer = nullptr;
    if (!CMBlockBufferIsRangeContiguous(block_buffer, 0, 0)) 
    {
        status = CMBlockBufferCreateContiguous(kCFAllocatorDefault, block_buffer, block_allocator, nullptr, 0, 0, 0, &contiguous_buffer);
        if (status != noErr) 
        {
            CFRelease(block_buffer);
            return false;
        }
    } 
    else 
    {
        contiguous_buffer = block_buffer;
        block_buffer = nullptr;
    }

    // Get a raw pointer into allocated memory.
    size_t block_buffer_size = 0;
    char* data_ptr = nullptr;
    status = CMBlockBufferGetDataPointer(contiguous_buffer, 0, nullptr, &block_buffer_size, &data_ptr);
    if (status != kCMBlockBufferNoErr) 
    {
        FAVResult::Log(EAVResult::Error, TEXT("Failed to flatten non-contiguous block buffer"), TEXT("VT"));
        CFRelease(contiguous_buffer);
        return false;
    }
    check(block_buffer_size == reader.BytesRemainingForAVC());

    // Write Avcc NALUs into block buffer memory.
    AvccBufferWriter writer(reinterpret_cast<uint8*>(data_ptr), block_buffer_size);
    while (reader.BytesRemaining() > 0) 
    {
        const uint8* nalu_data_ptr = nullptr;
        size_t nalu_data_size = 0;
        if (reader.ReadNalu(&nalu_data_ptr, &nalu_data_size)) 
        {
            writer.WriteNalu(nalu_data_ptr, nalu_data_size);
        }
    }

    // Create sample buffer.
    status = CMSampleBufferCreate(kCFAllocatorDefault, contiguous_buffer, true, nullptr, nullptr, video_format, 1, 0, nullptr, 0, nullptr, out_sample_buffer);
  
    if (status != noErr) 
    {
        FAVResult::Log(EAVResult::Error, TEXT("Failed to create sample buffer"), TEXT("VT"));
        CFRelease(contiguous_buffer);
        return false;
    }
  
    CFRelease(contiguous_buffer);
    return true;
}

CMVideoFormatDescriptionRef NaluRewriter::CreateH264VideoFormatDescription(const uint8* annexb_buffer, size_t annexb_buffer_size)
{
    const uint8* param_set_ptrs[2] = {};
    size_t param_set_sizes[2] = {};
    AnnexBBufferReader reader(annexb_buffer, annexb_buffer_size);
    // Skip everyting before the SPS, then read the SPS and PPS
    if (!reader.SeekToNextNaluOfType(H264::ENaluType::SequenceParameterSet)) 
    {
        return nullptr;
    }
    if (!reader.ReadNalu(&param_set_ptrs[0], &param_set_sizes[0])) 
    {
        FAVResult::Log(EAVResult::Error, TEXT("Failed to read SPS"), TEXT("VT"));
        return nullptr;
    }
    if (!reader.ReadNalu(&param_set_ptrs[1], &param_set_sizes[1])) 
    {
        FAVResult::Log(EAVResult::Error, TEXT("Failed to read PPS"), TEXT("VT"));
        return nullptr;
    }

    // Parse the SPS and PPS into a CMVideoFormatDescription.
    CMVideoFormatDescriptionRef description = nullptr;
    OSStatus status = CMVideoFormatDescriptionCreateFromH264ParameterSets(kCFAllocatorDefault, 2, param_set_ptrs, param_set_sizes, 4, &description);
    if (status != noErr) 
    {
        FAVResult::Log(EAVResult::Error, TEXT("Failed to create video format description"), TEXT("VT"));
        return nullptr;
    }
    return description;
}

bool NaluRewriter::H265CMSampleBufferToAnnexBBuffer(CMSampleBufferRef hvcc_sample_buffer,bool is_keyframe, TArray<uint8>& annexb_buffer) 
{
    check(hvcc_sample_buffer);

    // Get format description from the sample buffer.
    CMVideoFormatDescriptionRef description = CMSampleBufferGetFormatDescription(hvcc_sample_buffer);
    if (description == nullptr) 
    {
        FAVResult::Log(EAVResult::Error, TEXT("Failed to get sample buffer's description"), TEXT("VT"));
        return false;
    }

    // Get parameter set information.
    int nalu_header_size = 0;
    size_t param_set_count = 0;
    OSStatus status = CMVideoFormatDescriptionGetHEVCParameterSetAtIndex(description, 0, nullptr, nullptr, &param_set_count, &nalu_header_size);
    if (status != 0) 
    {
        FAVResult::Log(EAVResult::Error, TEXT("Failed to get parameter set"), TEXT("VT"));
        return false;
    }

    check(nalu_header_size == kAvccHeaderByteSize);
    check(param_set_count == 3);

    // Truncate any previous data in the buffer without changing its capacity.
    annexb_buffer.SetNum(0, EAllowShrinking::No);

    // Place all parameter sets at the front of buffer.
    if (is_keyframe) 
    {
        size_t param_set_size = 0;
        const uint8* param_set = nullptr;
        for (size_t i = 0; i < param_set_count; ++i) 
        {
            status = CMVideoFormatDescriptionGetHEVCParameterSetAtIndex(description, i, &param_set, &param_set_size, nullptr, nullptr);
            if (status != 0) 
            {
                FAVResult::Log(EAVResult::Error, TEXT("Failed to get parameter set"), TEXT("VT"));
                return false;
            }
            // Update buffer.
            annexb_buffer.Append(kAnnexBHeaderBytes, sizeof(kAnnexBHeaderBytes));
            annexb_buffer.Append(reinterpret_cast<const uint8*>(param_set), param_set_size);
        }
    }

    // Get block buffer from the sample buffer.
    CMBlockBufferRef block_buffer = CMSampleBufferGetDataBuffer(hvcc_sample_buffer);
    if (block_buffer == nullptr) 
    {
        FAVResult::Log(EAVResult::Error, TEXT("Failed to get sample buffer's block buffer"), TEXT("VT"));
        return false;
    }

    CMBlockBufferRef contiguous_buffer = nullptr;
    // Make sure block buffer is contiguous.
    if (!CMBlockBufferIsRangeContiguous(block_buffer, 0, 0)) 
    {
        status = CMBlockBufferCreateContiguous(nullptr, block_buffer, nullptr, nullptr, 0, 0, 0, &contiguous_buffer);
        if (status != 0) 
        {
            FAVResult::Log(EAVResult::Error, TEXT("Failed to flatten non-contiguous block buffer"), TEXT("VT"));
            return false;
        }
    } 
    else 
    {
        contiguous_buffer = block_buffer;
        // Retain to make cleanup easier.
        CFRetain(contiguous_buffer);
        block_buffer = nullptr;
    }

    // Now copy the actual data.
    char* data_ptr = nullptr;
    size_t block_buffer_size = CMBlockBufferGetDataLength(contiguous_buffer);
    status = CMBlockBufferGetDataPointer(contiguous_buffer, 0, nullptr, nullptr, &data_ptr);
    if (status != 0) 
    {
        FAVResult::Log(EAVResult::Error, TEXT("Failed to get block buffer data"), TEXT("VT"));
        CFRelease(contiguous_buffer);
        return false;
    }

    size_t bytes_remaining = block_buffer_size;
    while (bytes_remaining > 0) 
    {
        // The size type here must match |nalu_header_size|, we expect 4 bytes.
        // Read the length of the next packet of data. Must convert from big endian
        // to host endian.
        check(bytes_remaining >= (size_t)nalu_header_size);
        uint32_t* uint32_data_ptr = reinterpret_cast<uint32_t*>(data_ptr);
        uint32_t packet_size = CFSwapInt32BigToHost(*uint32_data_ptr);
        // Update buffer.
        annexb_buffer.Append(kAnnexBHeaderBytes, sizeof(kAnnexBHeaderBytes));
        annexb_buffer.Append(reinterpret_cast<uint8*>(data_ptr + (size_t)nalu_header_size), packet_size);

        size_t bytes_written = packet_size + sizeof(kAnnexBHeaderBytes);
        bytes_remaining -= bytes_written;
        data_ptr += bytes_written;
    }

    check(bytes_remaining == (size_t)0);

    CFRelease(contiguous_buffer);

    return true;
}

bool NaluRewriter::H265AnnexBBufferToCMSampleBuffer(const uint8* annexb_buffer, size_t annexb_buffer_size, CMVideoFormatDescriptionRef video_format, CMSampleBufferRef* out_sample_buffer, CMMemoryPoolRef memory_pool) 
{
    check(annexb_buffer);
    check(out_sample_buffer);
    check(video_format);
    *out_sample_buffer = nullptr;

    AnnexBBufferReader reader(annexb_buffer, annexb_buffer_size, false);
    if (reader.SeekToNextNaluOfType(H265::ENaluType::VPS_NUT)) 
    {
        // Buffer contains an SPS NALU - skip it and the following PPS
        const uint8* data;
        size_t data_len;
        if (!reader.ReadNalu(&data, &data_len)) 
        {
            FAVResult::Log(EAVResult::Error, TEXT("Failed to read VPS"), TEXT("VT"));
            return false;
        }
        if (!reader.ReadNalu(&data, &data_len)) 
        {
            FAVResult::Log(EAVResult::Error, TEXT("Failed to read SPS"), TEXT("VT"));
            return false;
        }
        if (!reader.ReadNalu(&data, &data_len)) 
        {
            FAVResult::Log(EAVResult::Error, TEXT("Failed to read PPS"), TEXT("VT"));
            return false;
        }
    } 
    else 
    {
        // No SPS NALU - start reading from the first NALU in the buffer
        reader.SeekToStart();
    }

    // Allocate memory as a block buffer.
    CMBlockBufferRef block_buffer = nullptr;
    CFAllocatorRef block_allocator = CMMemoryPoolGetAllocator(memory_pool);
    OSStatus status = CMBlockBufferCreateWithMemoryBlock(kCFAllocatorDefault, nullptr, reader.BytesRemaining(), block_allocator, nullptr, 0, reader.BytesRemaining(), kCMBlockBufferAssureMemoryNowFlag, &block_buffer);
    if (status != kCMBlockBufferNoErr) 
    {
        FAVResult::Log(EAVResult::Error, TEXT("Failed to create block buffer"), TEXT("VT"));
        return false;
    }

    // Make sure block buffer is contiguous.
    CMBlockBufferRef contiguous_buffer = nullptr;
    if (!CMBlockBufferIsRangeContiguous(block_buffer, 0, 0)) 
    {
        status = CMBlockBufferCreateContiguous(kCFAllocatorDefault, block_buffer, block_allocator, nullptr, 0, 0, 0, &contiguous_buffer);
        if (status != noErr) 
        {
            FAVResult::Log(EAVResult::Error, TEXT("Failed to flatten non-contiguous buffer"), TEXT("VT"));
            CFRelease(block_buffer);
            return false;
        }
    } 
    else 
    {
        contiguous_buffer = block_buffer;
        block_buffer = nullptr;
    }

    // Get a raw pointer into allocated memory.
    size_t block_buffer_size = 0;
    char* data_ptr = nullptr;
    status = CMBlockBufferGetDataPointer(contiguous_buffer, 0, nullptr, &block_buffer_size, &data_ptr);
    if (status != kCMBlockBufferNoErr) 
    {
        FAVResult::Log(EAVResult::Error, TEXT("Failed to get block buffer data ptr"), TEXT("VT"));
        CFRelease(contiguous_buffer);
        return false;
    }
    check(block_buffer_size == reader.BytesRemaining());

    // Write Avcc NALUs into block buffer memory.
    AvccBufferWriter writer(reinterpret_cast<uint8*>(data_ptr), block_buffer_size);
    while (reader.BytesRemaining() > 0) 
    {
        const uint8* nalu_data_ptr = nullptr;
        size_t nalu_data_size = 0;
        if (reader.ReadNalu(&nalu_data_ptr, &nalu_data_size)) 
        {
            writer.WriteNalu(nalu_data_ptr, nalu_data_size);
        }
    }

    // Create sample buffer.
    status = CMSampleBufferCreate(kCFAllocatorDefault, contiguous_buffer, true, nullptr, nullptr, video_format, 1, 0, nullptr, 0, nullptr, out_sample_buffer);
    if (status != noErr) 
    {
        FAVResult::Log(EAVResult::Error, TEXT("Failed to create sample buffer"), TEXT("VT"));
        CFRelease(contiguous_buffer);
        return false;
    }
    CFRelease(contiguous_buffer);
    return true;
}

CMVideoFormatDescriptionRef NaluRewriter::CreateH265VideoFormatDescription(const uint8* annexb_buffer, size_t annexb_buffer_size) 
{
    const uint8* param_set_ptrs[3] = {};
    size_t param_set_sizes[3] = {};
    AnnexBBufferReader reader(annexb_buffer, annexb_buffer_size, false);
    // Skip everyting before the VPS, then read the VPS, SPS and PPS
    if (!reader.SeekToNextNaluOfType(H265::ENaluType::VPS_NUT)) 
    {
        return nullptr;
    }
    
    if (!reader.ReadNalu(&param_set_ptrs[0], &param_set_sizes[0])) 
    {
        FAVResult::Log(EAVResult::Error, TEXT("Failed to read VPS"), TEXT("VT"));
        return nullptr;
    }
    
    if (!reader.ReadNalu(&param_set_ptrs[1], &param_set_sizes[1])) 
    {
        FAVResult::Log(EAVResult::Error, TEXT("Failed to read SPS"), TEXT("VT"));
        return nullptr;
    }
    
    if (!reader.ReadNalu(&param_set_ptrs[2], &param_set_sizes[2])) 
    {
        FAVResult::Log(EAVResult::Error, TEXT("Failed to read PPS"), TEXT("VT"));
        return nullptr;
    }

    // Parse the SPS and PPS into a CMVideoFormatDescription.
    CMVideoFormatDescriptionRef description = nullptr;
    OSStatus status = CMVideoFormatDescriptionCreateFromHEVCParameterSets(kCFAllocatorDefault, 3, param_set_ptrs, param_set_sizes, 4, nullptr, &description);
    if (status != noErr) 
    {
        FAVResult::Log(EAVResult::Error, TEXT("Failed to create video format description"), TEXT("VT"));
        return nullptr;
    }
    
    return description;
}

bool NaluRewriter::VP9BufferToCMSampleBuffer(const uint8* buffer, size_t buffer_size, CMVideoFormatDescriptionRef video_format, CMSampleBufferRef* out_sample_buffer, CMMemoryPoolRef memory_pool)
{
    check(buffer);
    check(out_sample_buffer);
    *out_sample_buffer = nullptr;

    CMBlockBufferRef block_buffer = nullptr;
    CFAllocatorRef block_allocator = CMMemoryPoolGetAllocator(memory_pool);
    OSStatus status = CMBlockBufferCreateWithMemoryBlock(kCFAllocatorDefault, NULL, buffer_size, block_allocator, NULL, 0, buffer_size, kCMBlockBufferAssureMemoryNowFlag, &block_buffer);
    if (status != kCMBlockBufferNoErr) {
        FAVResult::Log(EAVResult::Error, TEXT("VP9BufferToCMSampleBuffer CMBlockBufferCreateWithMemoryBlock failed"), TEXT("VT"));
        return false;
    }

    status = CMBlockBufferReplaceDataBytes(buffer, block_buffer, 0, buffer_size);
    if (status != noErr) 
    {
        FAVResult::Log(EAVResult::Error, TEXT("VP9BufferToCMSampleBuffer CMBlockBufferReplaceDataBytes failed"), TEXT("VT"));
        return false;
    }

    status = CMSampleBufferCreate(kCFAllocatorDefault, block_buffer, true, nullptr, nullptr, video_format, 1, 0, nullptr, 0, nullptr, out_sample_buffer);
    if (status != noErr) 
    {
        FAVResult::Log(EAVResult::Error, TEXT("VP9BufferToCMSampleBuffer CMSampleBufferCreate failed"), TEXT("VT"));
        return false;
    }

    CFRelease(block_buffer);
    return true;
}

CMVideoFormatDescriptionRef NaluRewriter::CreateVP9VideoFormatDescription(const uint8* rtp_buffer, size_t rtp_buffer_size)
{
    // Create a dummy decoder config for access to its parsing capabilities
    FVideoDecoderConfigVP9 VP9;

    TSharedPtr<uint8> const CopiedData = MakeShareable(new uint8[rtp_buffer_size]);
	FMemory::BigBlockMemcpy(CopiedData.Get(), rtp_buffer, rtp_buffer_size);

    VP9::Header_t Header;
	if (VP9.Parse(FVideoPacket(CopiedData, rtp_buffer_size, 0, 0, 0, false), Header) != EAVResult::Success)
    {
		FAVResult::Log(EAVResult::Error, TEXT("CreateVP9VideoFormatDescription failed to parse VP9 bitstream"), TEXT("VT"));
        return nullptr;
    }
	// Early exit if size is not set because this means a valid VP9VideoFormatDescription cannot be constructed.
	if (Header.frame_width_minus_1 <= 0)
	{
		return nullptr;
	}

    const size_t VPCodecConfigurationContentsSize = 12;

    uint8 ChromaSubSampling = 1;
    switch(Header.sub_sampling)
    {
        case VP9::ESubSampling::k444:
            ChromaSubSampling = 3;
			break;
        case VP9::ESubSampling::k440:
            ChromaSubSampling = 1;
			break;
        case VP9::ESubSampling::k422:
            ChromaSubSampling = 2;
			break;
        case VP9::ESubSampling::k420:
            ChromaSubSampling = 1;
			break;
    }

	uint8 BitDepthChromaAndRange = (0xF & static_cast<uint8>(Header.bit_depth.Value)) << 4 | (0x7 & ChromaSubSampling) << 1 | (0x1 & static_cast<uint8>(Header.color_range.Value));
    
    uint8 Record[VPCodecConfigurationContentsSize];
    memset((void*)Record, 0, VPCodecConfigurationContentsSize);
    // Version and flags (4 bytes)
    Record[0] = 1;
    // profile
    Record[4] = ((Header.profile_high_bit << 1) + Header.profile_low_bit);
    // level
    Record[5] = 10;
    // bitDepthChromaAndRange
    Record[6] = BitDepthChromaAndRange;
    // colourPrimaries
    Record[7] = 2; // Unspecified.
    // transferCharacteristics
    Record[8] = 2; // Unspecified.
    // matrixCoefficients
    Record[9] = 2; // Unspecified.

    CFDataRef Data = CFDataCreate(kCFAllocatorDefault, Record, VPCodecConfigurationContentsSize);

    CFMutableDictionaryRef AvcInfo = CFDictionaryCreateMutable(kCFAllocatorDefault, 1, &kCFTypeDictionaryKeyCallBacks, &kCFTypeDictionaryValueCallBacks);

    CFDictionarySetValue(AvcInfo, CFSTR("vpcC"), Data);

    CFMutableDictionaryRef ConfigInfo = CFDictionaryCreateMutable(kCFAllocatorDefault, 0, &kCFTypeDictionaryKeyCallBacks, &kCFTypeDictionaryValueCallBacks);
    CFDictionarySetValue(ConfigInfo, kCMFormatDescriptionExtension_SampleDescriptionExtensionAtoms, AvcInfo);

    CONDITIONAL_RELEASE(Data);
    CONDITIONAL_RELEASE(AvcInfo);

    CMVideoFormatDescriptionRef FormatDescription = nullptr;
	OSStatus Status = CMVideoFormatDescriptionCreate(kCFAllocatorDefault, kCMVideoCodecType_VP9, Header.frame_width_minus_1 + 1, Header.frame_height_minus_1 + 1, ConfigInfo, &FormatDescription);
    if (Status != noErr)
    { 
        FAVResult::Log(EAVResult::Error, TEXT("CreateVP9VideoFormatDescription failed to create video format description"), TEXT("VT"), Status);
        CONDITIONAL_RELEASE(ConfigInfo);
        return nullptr;
    };

    CONDITIONAL_RELEASE(ConfigInfo);

    return FormatDescription;
}

NaluRewriter::AnnexBBufferReader::AnnexBBufferReader(const uint8* annexb_buffer, size_t length, bool isH264)
    : start_(annexb_buffer), length_(length), offset_(offsets_.CreateIterator())
{
    check(annexb_buffer);

    if(isH264)
    {
         TArray<H264::FNaluH264> FoundNalus;
	     FAVResult Result = H264::FindNALUs(FVideoPacket(MakeShareable<uint8>(const_cast<uint8*>(annexb_buffer), FFakeDeleter()), length, 0, 0, 0, false), FoundNalus);

         for(H264::FNaluH264& Nal : FoundNalus)
         {
             offsets_.Add({ Nal.Start, Nal.Start + Nal.StartCodeSize, Nal.Size });
         }
    }
    else
    {
        TArray<H265::FNaluH265> FoundNalus;
        FAVResult Result = H265::FindNALUs(FVideoPacket(MakeShareable<uint8>(const_cast<uint8*>(annexb_buffer), FFakeDeleter()), length, 0, 0, 0, false), FoundNalus);
        
        for(H265::FNaluH265& Nal : FoundNalus)
        {
            offsets_.Add({ Nal.Start, Nal.Start + Nal.StartCodeSize, Nal.Size });
        }
    }
}

NaluRewriter::AnnexBBufferReader::~AnnexBBufferReader() = default;

bool NaluRewriter::AnnexBBufferReader::ReadNalu(const uint8** out_nalu, size_t* out_length)
{
    check(out_nalu);
    check(out_length);

    *out_nalu = nullptr;
    *out_length = 0;

    if (!offset_) {
        return false;
    }
    
    *out_nalu = start_ + offset_->payload_start_offset;
    *out_length = offset_->payload_size;
    ++offset_;

    return true;
}

size_t NaluRewriter::AnnexBBufferReader::BytesRemaining() const
{
    if (!offset_) 
    {
        return 0;
    }
    return length_ - offset_->start_offset;
}

size_t NaluRewriter::AnnexBBufferReader::BytesRemainingForAVC() const
{
    if (!offset_)
    {
        return 0;
    }
  
    auto iterator = offset_;
    size_t size = 0;
    while (iterator) 
    {
        size += kAvccHeaderByteSize + iterator->payload_size;
        iterator++;
    }
    return size;
}

void NaluRewriter::AnnexBBufferReader::SeekToStart()
{
    offset_.Reset();
}

bool NaluRewriter::AnnexBBufferReader::SeekToNextNaluOfType(UE::AVCodecCore::H264::ENaluType type)
{
    for (; offset_; ++offset_)
    {
        if (offset_->payload_size < 1)
        {
            continue;
        }
        if (ParseH264NaluType(*(start_ + offset_->payload_start_offset)) == type)
        {
            return true;
        }
    }
    return false;
}

bool NaluRewriter::AnnexBBufferReader::SeekToNextNaluOfType(UE::AVCodecCore::H265::ENaluType type) 
{
     for (; offset_; ++offset_) 
     {
         if (offset_->payload_size < 1)
         {
             continue;
         }
        
         if (ParseH265NaluType(*(start_ + offset_->payload_start_offset)) == type)
         {
             return true;
         }
     }
  
    return false;
}

H264::ENaluType NaluRewriter::AnnexBBufferReader::ParseH264NaluType(uint8 data) {
    return static_cast<UE::AVCodecCore::H264::ENaluType>(data & 0x1F);
}

UE::AVCodecCore::H265::ENaluType NaluRewriter::AnnexBBufferReader::ParseH265NaluType(uint8 data) {
    return static_cast<UE::AVCodecCore::H265::ENaluType>((data & 0x7E) >> 1);
}

NaluRewriter::AvccBufferWriter::AvccBufferWriter(uint8* const avcc_buffer, size_t length)
    : start_(avcc_buffer), offset_(0), length_(length)
{
  check(avcc_buffer);
}

bool NaluRewriter::AvccBufferWriter::WriteNalu(const uint8* data, size_t data_size) 
{
    // Check if we can write this length of data.
    if (data_size + kAvccHeaderByteSize > BytesRemaining()) 
    {
        return false;
    }
  
    // Write length header, which needs to be big endian.
    uint32_t big_endian_length = CFSwapInt32HostToBig(data_size);
    memcpy(start_ + offset_, &big_endian_length, sizeof(big_endian_length));
    offset_ += sizeof(big_endian_length);
    // Write data.
    memcpy(start_ + offset_, data, data_size);
    offset_ += data_size;
    return true;
}

size_t NaluRewriter::AvccBufferWriter::BytesRemaining() const
{
  return length_ - offset_;
}
