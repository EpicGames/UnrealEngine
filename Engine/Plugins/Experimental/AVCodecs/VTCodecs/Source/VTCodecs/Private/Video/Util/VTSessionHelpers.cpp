// Copyright Epic Games, Inc. All Rights Reserved.

#include "Video/Util/VTSessionHelpers.h"
#include "AVResult.h"

FString VTSessionHelpers::CFStringToString(const CFStringRef CfString) 
{    
    // Get the size needed for UTF8 plus terminating character.
    size_t BufferSize = CFStringGetMaximumSizeForEncoding(CFStringGetLength(CfString), kCFStringEncodingUTF8) + 1;

    TUniquePtr<char[]> Buffer(new char[BufferSize]);
    if (CFStringGetCString(CfString, Buffer.Get(), BufferSize, kCFStringEncodingUTF8)) 
    {
        // Copy over the characters.
        FString String(Buffer.Get());
        return String;
    }

    return "";
}

void VTSessionHelpers::SetVTSessionProperty(VTSessionRef Session, CFStringRef Key, int32_t Value) {
    CFNumberRef Num = CFNumberCreate(kCFAllocatorDefault, kCFNumberSInt32Type, &Value);
    OSStatus Status = VTSessionSetProperty(Session, Key, Num);
    CFRelease(Num);
    if (Status != 0) 
    {
        FString KeyString = CFStringToString(Key);
        FAVResult::Log(EAVResult::Warning, FString::Printf(TEXT("VTSessionSetProperty failed to set: %s to %d"), *KeyString, Value), TEXT("VT"), Status);
    }
}

void VTSessionHelpers::SetVTSessionProperty(VTSessionRef Session, CFStringRef Key, bool Value) 
{
    CFBooleanRef Bool = (Value) ? kCFBooleanTrue : kCFBooleanFalse;
    OSStatus Status = VTSessionSetProperty(Session, Key, Bool);
    if (Status != 0) 
    {
        FString KeyString = CFStringToString(Key);
        FAVResult::Log(EAVResult::Warning, FString::Printf(TEXT("VTSessionSetProperty failed to set: %s to %d"), *KeyString, Value), TEXT("VT"), Status);
    }
}

void VTSessionHelpers::SetVTSessionProperty(VTSessionRef Session, CFStringRef Key, CFStringRef Value) 
{
    OSStatus Status = VTSessionSetProperty(Session, Key, Value);
    if (Status != 0) 
    {
        FString KeyString = CFStringToString(Key);
        FString ValueString = CFStringToString(Value);
        FAVResult::Log(EAVResult::Warning, FString::Printf(TEXT("VTSessionSetProperty failed to set: %s to %s"), *KeyString, *ValueString), TEXT("VT"), Status);
    }
}
