// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AVUtility.h"

#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
#endif

THIRD_PARTY_INCLUDES_START
#include <VideoToolbox/VideoToolbox.h>
THIRD_PARTY_INCLUDES_END

class VTCODECS_API FVT : public FAPI
{
public:
	bool bHasCompatibleGPU = false;

	FVT();

	virtual bool IsValid() const override;
private:
	bool bIsIoSurfaceAvailable;
};

DECLARE_TYPEID(FVT, VTCODECS_API);
