// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AVExtension.h"
#include "Video/VideoDecoder.h"
#include "VT.h"

THIRD_PARTY_INCLUDES_START
#include <VideoToolbox/VideoToolbox.h>
THIRD_PARTY_INCLUDES_END

struct VTCODECS_API FVideoDecoderConfigVT : public FAVConfig
{
public:
	static TAVResult<OSType> ConvertFormat(EVideoFormat const& Format);
	
public:
	CMVideoCodecType Codec;
    CMVideoFormatDescriptionRef VideoFormat = nullptr;

	FVideoDecoderConfigVT()
		: FAVConfig()
	{

	}

	~FVideoDecoderConfigVT()
	{
		SetVideoFormat(nullptr);
	}
		
	FVideoDecoderConfigVT(FVideoDecoderConfigVT const& Other)
	{
		*this = Other;
		if (VideoFormat)
		{
			CFRetain(VideoFormat);
		}
	}
    
    void SetVideoFormat(CMVideoFormatDescriptionRef Format)
    {
        if(VideoFormat == Format)
        {
            return;
        }

        if (VideoFormat)
        {
            CFRelease(VideoFormat);
            VideoFormat = nullptr;
        }

        VideoFormat = Format;
        if (VideoFormat)
        {
            CFRetain(VideoFormat);
        }
    }

	FVideoDecoderConfigVT& operator=(FVideoDecoderConfigVT const& Other)
	{
		FMemory::Memcpy(*this, Other);
		if (VideoFormat)
		{
			CFRetain(VideoFormat);
		}

		return *this;
	}

	bool operator==(FVideoDecoderConfigVT const& Other) const
	{
        return this->Codec == Other.Codec &&
               CMFormatDescriptionEqual(this->VideoFormat, Other.VideoFormat);
	}

	bool operator!=(FVideoDecoderConfigVT const& Other) const
	{
		return !(*this == Other);
	}
};

template <>
FAVResult FAVExtension::TransformConfig(FVideoDecoderConfigVT& OutConfig, struct FVideoDecoderConfig const& InConfig);

template <>
FAVResult FAVExtension::TransformConfig(struct FVideoDecoderConfig& OutConfig, FVideoDecoderConfigVT const& InConfig);

template <>
FAVResult FAVExtension::TransformConfig(FVideoDecoderConfigVT& OutConfig, struct FVideoDecoderConfigH264 const& InConfig);

template <>
FAVResult FAVExtension::TransformConfig(FVideoDecoderConfigVT& OutConfig, struct FVideoDecoderConfigH265 const& InConfig);

template <>
FAVResult FAVExtension::TransformConfig(FVideoDecoderConfigVT& OutConfig, struct FVideoDecoderConfigVP9 const& InConfig);

DECLARE_TYPEID(FVideoDecoderConfigVT, VTCODECS_API);
