// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Video/VideoDecoder.h"
#include "Video/Decoders/Configs/VideoDecoderConfigVT.h"
#include "Video/Resources/Metal/VideoResourceMetal.h"

#include "Containers/Queue.h"
#include "HAL/Platform.h"
#include "VT.h"
#include "Video/Util/VTSessionHelpers.h"

THIRD_PARTY_INCLUDES_START
#include <VideoToolbox/VideoToolbox.h>
#include <CoreMedia/CoreMedia.h>
THIRD_PARTY_INCLUDES_END

class FVideoDecoderVT : public TVideoDecoder<FVideoResourceMetal, FVideoDecoderConfigVT>
{
private:
    VTDecompressionSessionRef Decoder = nullptr;
    CVMetalTextureCacheRef TextureCache = nullptr;
    CMMemoryPoolRef MemoryPool;

	uint8 bInitialized : 1;

	uint64 FrameCount = 0;

	bool bIsOpen = false;

    class DecodeParams
    {
    public:
        DecodeParams() {}
        DecodeParams(CMTime Timestamp)
            : Timestamp(Timestamp) {}
        
        CMTime Timestamp;
    };

    class FFrame
	{
    public:
        FFrame() {}
        
        FFrame(CVPixelBufferRef InImageBuffer, CMTime InTimestamp, CMTime InDuration)
            : ImageBuffer(InImageBuffer), Timestamp(InTimestamp), Duration(InDuration)
        {
            if(ImageBuffer)
            {
                CVPixelBufferRetain(ImageBuffer);
            }
        }
        
        ~FFrame()
        {
            if(ImageBuffer)
            {
                CVPixelBufferRelease(ImageBuffer);
            }
        }
        
        CVPixelBufferRef ImageBuffer;
        CMTime Timestamp;
        CMTime Duration;
	};

    TQueue<TSharedPtr<FFrame>> Frames;

public:	
	FVideoDecoderVT() = default;
	virtual ~FVideoDecoderVT() override;

	virtual bool IsOpen() const override;
	virtual FAVResult Open(TSharedRef<FAVDevice> const& NewDevice, TSharedRef<FAVInstance> const& NewInstance) override;
	virtual void Close() override;
    void DestroyDecompressionSession();
    void ConfigureDecompressionSession();

	bool IsInitialized() const;

	virtual FAVResult ApplyConfig() override;
	
	virtual FAVResult ReceiveFrame(TResolvableVideoResource<FVideoResourceMetal>& InOutResource) override;
    FAVResult HandleFrame(void* Params, OSStatus Status, VTDecodeInfoFlags InfoFlags, CVImageBufferRef ImageBuffer, CMTime Timestamp, CMTime Duration);

	virtual FAVResult SendPacket(FVideoPacket const& Packet) override;
};

namespace Internal 
{
    void VTDecompressionOutputCallback(void* Decoder, void* Params, OSStatus Status, VTDecodeInfoFlags InfoFlags, CVImageBufferRef ImageBuffer, CMTime Timestamp, CMTime Duration);
}
