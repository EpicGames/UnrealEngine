// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AVExtension.h"
#include "Video/VideoEncoder.h"
#include "VT.h"

THIRD_PARTY_INCLUDES_START
#include <VideoToolbox/VideoToolbox.h>
THIRD_PARTY_INCLUDES_END

struct VTCODECS_API FVideoEncoderConfigVT : public FAVConfig
{
public:
	static TAVResult<OSType> ConvertFormat(EVideoFormat const& Format);
	
public:
	uint32 Width = 1920;
	uint32 Height = 1080;
	uint32 FrameRate = 60;
	int32 TargetBitrate = 10000000;
	int32 MaxBitrate = 20000000;
	ERateControlMode RateControlMode = ERateControlMode::CBR;
	uint8 bFillData : 1;
	uint32 KeyframeInterval = 0;
	CMVideoCodecType Codec;
	CFStringRef Profile;
    EVideoFormat PixelFormat;
    CFStringRef EntropyCodingMode;
    int32 MinQuality;
    int32 MaxQuality;

	FVideoEncoderConfigVT()
		: FAVConfig()
	{

	}
		
	FVideoEncoderConfigVT(FVideoEncoderConfigVT const& Other)
	{
		*this = Other;
	}

	FVideoEncoderConfigVT& operator=(FVideoEncoderConfigVT const& Other)
	{
		FMemory::Memcpy(*this, Other);

		return *this;
	}

	bool operator==(FVideoEncoderConfigVT const& Other) const
	{
        return  this->Width == Other.Width &&
                this->Height == Other.Height &&
                this->Codec == Other.Codec &&
                this->Profile == Other.Profile && 
				this->FrameRate == Other.FrameRate &&
				this->TargetBitrate == Other.TargetBitrate &&
				this->MaxBitrate == Other.MaxBitrate &&
				this->RateControlMode == Other.RateControlMode &&
				this->bFillData == Other.bFillData &&
				this->KeyframeInterval == Other.KeyframeInterval &&
				this->Preset == Other.Preset &&
				this->TargetBitrate == Other.TargetBitrate &&
                this->PixelFormat == Other.PixelFormat &&
                this->EntropyCodingMode == Other.EntropyCodingMode &&
                this->MinQuality == Other.MinQuality &&
                this->MaxQuality == Other.MaxQuality;
	}

	bool operator!=(FVideoEncoderConfigVT const& Other) const
	{
		return !(*this == Other);
	}

    int64 GetCVPixelFormatType() const
    {
        switch(this->PixelFormat)
        {
            case EVideoFormat::BGRA:
                return kCVPixelFormatType_32BGRA;
            case EVideoFormat::ABGR10:
                return kCVPixelFormatType_ARGB2101010LEPacked;
            default:
                unimplemented();
                return 0;
        }
    }
};

template <>
FAVResult FAVExtension::TransformConfig(FVideoEncoderConfigVT& OutConfig, struct FVideoEncoderConfig const& InConfig);

template <>
FAVResult FAVExtension::TransformConfig(struct FVideoEncoderConfig& OutConfig, FVideoEncoderConfigVT const& InConfig);

template <>
FAVResult FAVExtension::TransformConfig(FVideoEncoderConfigVT& OutConfig, struct FVideoEncoderConfigH264 const& InConfig);

template <>
FAVResult FAVExtension::TransformConfig(FVideoEncoderConfigVT& OutConfig, struct FVideoEncoderConfigH265 const& InConfig);

DECLARE_TYPEID(FVideoEncoderConfigVT, VTCODECS_API);
