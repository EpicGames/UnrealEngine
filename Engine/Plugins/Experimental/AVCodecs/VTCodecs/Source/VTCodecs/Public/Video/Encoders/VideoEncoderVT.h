// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Video/VideoEncoder.h"
#include "Video/Encoders/Configs/VideoEncoderConfigVT.h"
#include "Video/Resources/Metal/VideoResourceMetal.h"

#include "Video/Decoders/Configs/VideoDecoderConfigH264.h"
#include "Video/Decoders/Configs/VideoDecoderConfigH265.h"

#include "Containers/Queue.h"
#include "HAL/Platform.h"
#include "VT.h"

#include "Video/Util/VTSessionHelpers.h"

THIRD_PARTY_INCLUDES_START
#include <VideoToolbox/VideoToolbox.h>
THIRD_PARTY_INCLUDES_END

template <typename TResource>
class TVideoEncoderVT : public TVideoEncoder<TResource, FVideoEncoderConfigVT>
{
private:
	VTCompressionSessionRef Encoder = nullptr;

    // Create a dummy decoder config so we have access to its parsing capabilities
    FVideoDecoderConfigH264 H264;
    FVideoDecoderConfigH265 H265;

	uint8 bInitialized : 1;

	uint64 FrameCount = 0;

	TQueue<FVideoPacket> Packets;

	bool bIsOpen = false;
    
    class EncodeParams
    {
    public:
        EncodeParams() {}
        EncodeParams(CMVideoCodecType Codec, CMTime Timestamp)
            : Codec(Codec), Timestamp(Timestamp) {}
        
        CMVideoCodecType Codec;
        CMTime Timestamp;
    };

public:	
	TVideoEncoderVT() = default;
	virtual ~TVideoEncoderVT() override;

	virtual bool IsOpen() const override;
	virtual FAVResult Open(TSharedRef<FAVDevice> const& NewDevice, TSharedRef<FAVInstance> const& NewInstance) override;
	virtual void Close() override;

	bool IsInitialized() const;

	virtual FAVResult ApplyConfig() override;
	FAVResult ConfigureCompressionSession(FVideoEncoderConfigVT const& Config);
	FAVResult SetEncoderBitrate(FVideoEncoderConfigVT const& Config);
	
	virtual FAVResult SendFrame(TSharedPtr<FVideoResourceMetal> const& Resource, uint32 Timestamp, bool bForceKeyframe = false) override;

	virtual FAVResult ReceivePacket(FVideoPacket& OutPacket) override;

    FAVResult HandlePacket(void* Params, OSStatus Status, VTEncodeInfoFlags InfoFlags, CMSampleBufferRef& SampleBuffer);
};

namespace Internal 
{
    void VTCompressionOutputCallback(void* Encoder, void* Params, OSStatus Status, VTEncodeInfoFlags InfoFlags, CMSampleBufferRef SampleBuffer);
}

#include "VideoEncoderVT.hpp"
