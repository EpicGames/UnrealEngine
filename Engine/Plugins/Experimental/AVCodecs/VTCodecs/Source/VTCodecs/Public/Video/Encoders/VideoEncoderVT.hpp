// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Video/Resources/Metal/VideoResourceMetal.h"
#include "Video/Util/NaluRewriter.h"

THIRD_PARTY_INCLUDES_START
#include <VideoToolbox/VideoToolbox.h>
#include <CoreMedia/CMSync.h>
THIRD_PARTY_INCLUDES_END

#define CONDITIONAL_RELEASE(x) \
    if (x)                     \
    {                          \
        CFRelease(x);          \
        x = nullptr;           \
    }

template <typename TResource>
TVideoEncoderVT<TResource>::~TVideoEncoderVT() 
{
    Close();
}

template <typename TResource>
bool TVideoEncoderVT<TResource>::IsOpen() const
{
    return bIsOpen;
}

template <typename TResource>
FAVResult TVideoEncoderVT<TResource>::Open(TSharedRef<FAVDevice> const& NewDevice, TSharedRef<FAVInstance> const& NewInstance)
{
    Close();

    TVideoEncoder<TResource, FVideoEncoderConfigVT>::Open(NewDevice, NewInstance);

    FrameCount = 0;

    bIsOpen = true;

    return EAVResult::Success;
}

template <typename TResource>
void TVideoEncoderVT<TResource>::Close()
{
    if (Encoder)
    {
        VTCompressionSessionInvalidate(Encoder);
        CFRelease(Encoder);
        Encoder = nullptr;
    }

    bIsOpen = false;
}

template <typename TResource>
bool TVideoEncoderVT<TResource>::IsInitialized() const
{
    return Encoder != nullptr;
}

template <typename TResource>
FAVResult TVideoEncoderVT<TResource>::ApplyConfig()
{
    if (IsOpen())
    {
        FVideoEncoderConfigVT const &PendingConfig = this->GetPendingConfig();
        if (this->AppliedConfig != PendingConfig)
        {
            if (IsInitialized())
            {
                if (this->AppliedConfig.Width == PendingConfig.Width && this->AppliedConfig.Height == PendingConfig.Height && this->AppliedConfig.Codec == PendingConfig.Codec && this->AppliedConfig.RateControlMode == PendingConfig.RateControlMode && this->AppliedConfig.KeyframeInterval == PendingConfig.KeyframeInterval && this->AppliedConfig.Profile == PendingConfig.Profile && this->AppliedConfig.PixelFormat == PendingConfig.PixelFormat && this->AppliedConfig.EntropyCodingMode == PendingConfig.EntropyCodingMode && this->AppliedConfig.MinQuality == PendingConfig.MinQuality && this->AppliedConfig.MaxQuality == PendingConfig.MaxQuality)
                {
                    // Only bitrates can be configured on the fly
                    SetEncoderBitrate(PendingConfig);
                }
                else
                {
                    if (Encoder)
                    {
                        VTCompressionSessionInvalidate(Encoder);
                        CFRelease(Encoder);
                        Encoder = nullptr;
                        FAVResult::Log(EAVResult::Success, TEXT("Re-initializing encoding session"), TEXT("VT"));
                    }
                }
            }

            if (!IsInitialized())
            {
                // Set source image buffer attributes. These attributes will be present on
                // buffers retrieved from the encoder's pixel buffer pool.
                const size_t AttributesSize = 3;
                CFTypeRef Keys[AttributesSize] = {
                    kCVPixelBufferOpenGLCompatibilityKey,
                    kCVPixelBufferIOSurfacePropertiesKey,
                    kCVPixelBufferPixelFormatTypeKey};

                CFDictionaryRef IOSurfaceValue = CFDictionaryCreate(kCFAllocatorDefault, nullptr, nullptr, 0, &kCFTypeDictionaryKeyCallBacks, &kCFTypeDictionaryValueCallBacks);

                int64_t PixelType = PendingConfig.GetCVPixelFormatType();
                CFNumberRef PixelFormat = CFNumberCreate(nullptr, kCFNumberLongType, &PixelType);

                CFTypeRef Values[AttributesSize] = {kCFBooleanTrue, IOSurfaceValue, PixelFormat};
                CFDictionaryRef SourceAttributes = CFDictionaryCreate(kCFAllocatorDefault, Keys, Values, AttributesSize, &kCFTypeDictionaryKeyCallBacks, &kCFTypeDictionaryValueCallBacks);

                CONDITIONAL_RELEASE(IOSurfaceValue);
                CONDITIONAL_RELEASE(PixelFormat);

                // Encoder specifications
                const size_t EncoderSpecsSize = 1;
                CFTypeRef EncoderKeys[EncoderSpecsSize] =
                    {
                        // We want HW acceleration for best latency, if we can't get it then fail creating the session
                        kVTVideoEncoderSpecification_RequireHardwareAcceleratedVideoEncoder,
                    };

                CFTypeRef EncoderValues[EncoderSpecsSize] =
                    {
                        kCFBooleanTrue};

                CFDictionaryRef EncoderSpec = CFDictionaryCreate(kCFAllocatorDefault, EncoderKeys, EncoderValues, EncoderSpecsSize, &kCFTypeDictionaryKeyCallBacks, &kCFTypeDictionaryValueCallBacks);

                OSStatus Result = VTCompressionSessionCreate(kCFAllocatorDefault,
                                                             PendingConfig.Width,
                                                             PendingConfig.Height,
                                                             PendingConfig.Codec,
                                                             EncoderSpec,
                                                             SourceAttributes,
                                                             NULL /* Default Compressed Data Allocator */,
                                                             Internal::VTCompressionOutputCallback,
                                                             this,
                                                             &Encoder);

                CONDITIONAL_RELEASE(SourceAttributes);
                CONDITIONAL_RELEASE(EncoderSpec);

                if (Result != 0)
                {
                    if (Result == kVTInvalidSessionErr)
                    {
                        FAVResult::Log(EAVResult::Warning, TEXT("Invalid VTCompressionSession. You could be exceeding the maximum resolution supported by VideoToolbox!"), TEXT("VT"));
                    }
                    return FAVResult(EAVResult::ErrorCreating, TEXT("Failed to create VTCompressionSession"), TEXT("VT"), Result);
                }

                ConfigureCompressionSession(PendingConfig);
            }
        }

        return TVideoEncoder<TResource, FVideoEncoderConfigVT>::ApplyConfig();
    }

    return FAVResult(EAVResult::ErrorInvalidState, TEXT("Encoder not open"), TEXT("VT"));
}

template <typename TResource>
FAVResult TVideoEncoderVT<TResource>::ConfigureCompressionSession(FVideoEncoderConfigVT const& Config)
{
    VTSessionHelpers::SetVTSessionProperty(Encoder, kVTCompressionPropertyKey_RealTime, true);

    VTSessionHelpers::SetVTSessionProperty(Encoder, kVTCompressionPropertyKey_ProfileLevel, Config.Profile);

    SetEncoderBitrate(Config);

    // Enable temporal compression, e.g. creating delta frames
    VTSessionHelpers::SetVTSessionProperty(Encoder, kVTCompressionPropertyKey_AllowTemporalCompression, true);

    // Frame reordering is unhelpful in WebRTC, real-time streaming usecases
    VTSessionHelpers::SetVTSessionProperty(Encoder, kVTCompressionPropertyKey_AllowFrameReordering, false);

    VTSessionHelpers::SetVTSessionProperty(Encoder, kVTCompressionPropertyKey_MaxKeyFrameInterval, (int32_t)Config.KeyframeInterval);

    VTSessionHelpers::SetVTSessionProperty(Encoder, kVTCompressionPropertyKey_MaxKeyFrameIntervalDuration, (Config.KeyframeInterval > 0 ? (int32_t)(Config.KeyframeInterval / Config.FrameRate) : 0));

    int32_t MinQP = (1.0f - (Config.MaxQuality / 100.0f)) * 51.0f;
    VTSessionHelpers::SetVTSessionProperty(Encoder, kVTCompressionPropertyKey_MinAllowedFrameQP, MinQP);

    int32_t MaxQP = (1.0f - (Config.MinQuality / 100.0f)) * 51.0f;
    VTSessionHelpers::SetVTSessionProperty(Encoder, kVTCompressionPropertyKey_MaxAllowedFrameQP, MaxQP);

    if (Config.Codec == kCMVideoCodecType_H264)
    {
        VTSessionHelpers::SetVTSessionProperty(Encoder, kVTCompressionPropertyKey_H264EntropyMode, Config.EntropyCodingMode);
    }

    return EAVResult::Success;
}

template <typename TResource>
FAVResult TVideoEncoderVT<TResource>::SetEncoderBitrate(FVideoEncoderConfigVT const &Config)
{
    VTSessionHelpers::SetVTSessionProperty(Encoder, kVTCompressionPropertyKey_AverageBitRate, Config.TargetBitrate);

    return EAVResult::Success;
}

template <typename TResource>
FAVResult TVideoEncoderVT<TResource>::SendFrame(TSharedPtr<FVideoResourceMetal> const &Resource, uint32 Timestamp, bool bForceKeyframe)
{
    if (IsOpen())
    {
        FVideoEncoderConfigVT &PendingConfig = this->EditPendingConfig();
        PendingConfig.PixelFormat = Resource->GetFormat();

        FAVResult AVResult = ApplyConfig();
        if (AVResult.IsNotSuccess())
        {
            return AVResult;
        }

        if (Resource.IsValid())
        {
            float TimestampSecs = Timestamp / 1000000.0f;
            CMTime PresentationTime = CMTimeMakeWithSeconds(TimestampSecs, NSEC_PER_SEC);
            CFDictionaryRef FrameProperties = nullptr;
            if (bForceKeyframe)
            {
                CFTypeRef Keys[] = {kVTEncodeFrameOptionKey_ForceKeyFrame};
                CFTypeRef Values[] = {kCFBooleanTrue};
                FrameProperties = CFDictionaryCreate(kCFAllocatorDefault, Keys, Values, 1, &kCFTypeDictionaryKeyCallBacks, &kCFTypeDictionaryValueCallBacks);
            }

            TUniquePtr<EncodeParams> Params = MakeUnique<EncodeParams>();
            Params.Reset(new EncodeParams(PendingConfig.Codec, PresentationTime));
			
			/*
			 * NOTE (william.belcher): If the texture passed in has was created with 'ETextureCreateFlags::CPUReadback', we can simply copy the raw bytes
			 * out of the texture. This is the preferred way as it prevents an ~50ms block on the encoding thread
			 */
		
			CFMutableDictionaryRef SourceAttributes = CFDictionaryCreateMutable(kCFAllocatorDefault, 1, &kCFTypeDictionaryKeyCallBacks, &kCFTypeDictionaryValueCallBacks);
			CFDictionarySetValue(SourceAttributes, kCVPixelBufferOpenGLCompatibilityKey, kCFBooleanTrue);

			CFDictionaryRef IOSurfaceValue = CFDictionaryCreate(kCFAllocatorDefault, nullptr, nullptr, 0, &kCFTypeDictionaryKeyCallBacks, &kCFTypeDictionaryValueCallBacks);
			CFDictionarySetValue(SourceAttributes, kCVPixelBufferIOSurfacePropertiesKey, IOSurfaceValue);

			int64 PixelType = 0;
			switch(Resource->GetDescriptor().Format)
			{
				case EVideoFormat::BGRA:
					PixelType = kCVPixelFormatType_32BGRA;
					break;
				case EVideoFormat::ABGR10:
					PixelType = kCVPixelFormatType_ARGB2101010LEPacked;
					break;
				default:
					checkNoEntry();
			}

			CFNumberRef PixelFormat = CFNumberCreate(nullptr, kCFNumberLongType, &PixelType);
			CFDictionarySetValue(SourceAttributes, kCVPixelBufferPixelFormatTypeKey, PixelFormat);

			CONDITIONAL_RELEASE(IOSurfaceValue);
			CONDITIONAL_RELEASE(PixelFormat);

			CVPixelBufferRef PixelBuffer;
			CVReturn Result = CVPixelBufferCreate(kCFAllocatorDefault, Resource->GetDescriptor().Width, Resource->GetDescriptor().Height, PixelType, SourceAttributes, &PixelBuffer);
			if (Result != kCVReturnSuccess)
			{
				return FAVResult(EAVResult::Error, TEXT("Failed to create CVPixelBufferRef"), TEXT("RHI"), Result);
			}
			CONDITIONAL_RELEASE(SourceAttributes);

			Result = CVPixelBufferLockBaseAddress(PixelBuffer, 0);
			if (Result != kCVReturnSuccess)
			{
				return FAVResult(EAVResult::Error, TEXT("Failed to lock base address"), TEXT("RHI"), Result);
			}

			// NOTE (belchy06): GetBytes assumes the raw texture has been created with with TexCreate_CPUReadback
			Resource->GetRaw()->getBytes(reinterpret_cast<uint8*>(CVPixelBufferGetBaseAddressOfPlane(PixelBuffer, 0)), CVPixelBufferGetBytesPerRow(PixelBuffer), MTL::Region(0, 0, Resource->GetDescriptor().Width, Resource->GetDescriptor().Height), 0);
			CVPixelBufferUnlockBaseAddress(PixelBuffer, 0);
			
            OSStatus Status = VTCompressionSessionEncodeFrame(Encoder, PixelBuffer, PresentationTime, kCMTimeInvalid, FrameProperties, (void *)Params.Release(), NULL);

            CONDITIONAL_RELEASE(FrameProperties);
			
			CONDITIONAL_RELEASE(PixelBuffer);

            if (Status != kCVReturnSuccess)
            {
                return FAVResult(EAVResult::Error, TEXT("Failed to encode"), TEXT("VT"), Status);
            }

            return EAVResult::Success;
        }
        else
        {
            // Flush encoder
            OSStatus Status = VTCompressionSessionCompleteFrames(Encoder, kCMTimeInvalid);
            if (Status != kCVReturnSuccess)
            {
                return FAVResult(EAVResult::Error, TEXT("Failed to flush"), TEXT("VT"), Status);
            }

            return EAVResult::Success;
        }
    }

    return FAVResult(EAVResult::ErrorInvalidState, TEXT("Encoder not open"), TEXT("VT"));
}

template <typename TResource>
FAVResult TVideoEncoderVT<TResource>::ReceivePacket(FVideoPacket &OutPacket)
{
    if (IsOpen())
    {
        if (Packets.Dequeue(OutPacket))
        {
            return EAVResult::Success;
        }

        return EAVResult::PendingInput;
    }

    return FAVResult(EAVResult::ErrorInvalidState, TEXT("Encoder not open"), TEXT("VT"));
}

template <typename TResource>
FAVResult TVideoEncoderVT<TResource>::HandlePacket(void *Params, OSStatus Status, VTEncodeInfoFlags InfoFlags, CMSampleBufferRef &SampleBuffer)
{
    if (IsOpen())
    {
        if (Status != 0)
        {
            return FAVResult(EAVResult::Error, TEXT("Failed to encode"), TEXT("VT"), Status);
        }

        if (InfoFlags & kVTEncodeInfo_FrameDropped)
        {
            return FAVResult(EAVResult::Error, TEXT("Frame dropped"), TEXT("VT"), Status);
        }

        bool bIsKeyframe = false;
        CFArrayRef Attachments = CMSampleBufferGetSampleAttachmentsArray(SampleBuffer, 0);
        if (Attachments != nullptr && CFArrayGetCount(Attachments))
        {
            CFDictionaryRef Attachment = static_cast<CFDictionaryRef>(CFArrayGetValueAtIndex(Attachments, 0));
            bIsKeyframe = !CFDictionaryContainsKey(Attachment, kCMSampleAttachmentKey_NotSync);
        }

        // SampleBuffer is an MPEG4 container stream, but we need to convert it to its raw bitstream
        EncodeParams *Config = static_cast<EncodeParams *>(Params);

        int QP = 0;
        TArray<uint8> Bitstream;
        if (Config->Codec == kCMVideoCodecType_H264)
        {
            if (!NaluRewriter::H264CMSampleBufferToAnnexBBuffer(SampleBuffer, bIsKeyframe, Bitstream))
            {
                return FAVResult(EAVResult::Error, TEXT("Failed to extract bitstream"), TEXT("VT"));
            }
        }
        else if (Config->Codec == kCMVideoCodecType_HEVC)
        {
            if (!NaluRewriter::H265CMSampleBufferToAnnexBBuffer(SampleBuffer, bIsKeyframe, Bitstream))
            {
                return FAVResult(EAVResult::Error, TEXT("Failed to extract bitstream"), TEXT("VT"));
            }
        }
        else
        {
            return FAVResult(EAVResult::ErrorUnsupported, TEXT("Unsupported codec"), TEXT("VT"));
        }

        TSharedPtr<uint8> const CopiedData = MakeShareable(new uint8[Bitstream.Num()]);
        FMemory::BigBlockMemcpy(CopiedData.Get(), Bitstream.GetData(), Bitstream.Num());

        if (Config->Codec == kCMVideoCodecType_H264)
        {
            using namespace UE::AVCodecCore::H264;
            TArray<Slice_t> Slices;
            FAVResult Result = H264.Parse(FVideoPacket(CopiedData, Bitstream.Num(), 0, 0, 0, false), Slices);
            if (Result != EAVResult::Success)
            {
                return FAVResult(EAVResult::Error, TEXT("Failed to parse bitstream"), TEXT("VT"), Result);
            }

            QP = H264.GetLastSliceQP(Slices).Get(0);
        }
        else if (Config->Codec == kCMVideoCodecType_HEVC)
        {
            using namespace UE::AVCodecCore::H265;
            TArray<Slice_t> Slices;
            FAVResult Result = H265.Parse(FVideoPacket(CopiedData, Bitstream.Num(), 0, 0, 0, false), Slices);
            if (Result != EAVResult::Success)
            {
                return FAVResult(EAVResult::Error, TEXT("Failed to parse bitstream"), TEXT("VT"), Result);
            }

            QP = H265.GetLastSliceQP(Slices).Get(0);
            FAVResult::Log(EAVResult::Warning, TEXT("H265 QP"), TEXT("VT"), QP);
        }

        return Packets.Enqueue(FVideoPacket(CopiedData, Bitstream.Num(), CMClockConvertHostTimeToSystemUnits(Config->Timestamp), ++FrameCount, (uint32_t)QP, bIsKeyframe)) ? EAVResult::Success : EAVResult::Error;
    }

    return FAVResult(EAVResult::ErrorInvalidState, TEXT("Encoder not open"), TEXT("VT"));
}
