// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Video/CodecUtils/CodecUtilsH264.h"
#include "Video/CodecUtils/CodecUtilsH265.h"

#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
#endif

THIRD_PARTY_INCLUDES_START
#include <CoreMedia/CoreMedia.h>
THIRD_PARTY_INCLUDES_END

#include <vector>

using namespace UE::AVCodecCore;

class NaluRewriter
{
public:
    // Converts a sample buffer emitted from the VideoToolbox encoder into a buffer
    // suitable for RTP. The sample buffer is in avcc format whereas the rtp buffer
    // needs to be in Annex B format. Data is written directly to |annexb_buffer|.
    static bool H264CMSampleBufferToAnnexBBuffer(CMSampleBufferRef avcc_sample_buffer, bool is_keyframe, TArray<uint8>& annexb_buffer);

    // Converts a buffer received from RTP into a sample buffer suitable for the
    // VideoToolbox decoder. The RTP buffer is in annex b format whereas the sample
    // buffer is in avcc format.
    // If |is_keyframe| is true then |video_format| is ignored since the format will
    // be read from the buffer. Otherwise |video_format| must be provided.
    // Caller is responsible for releasing the created sample buffer.
    static bool H264AnnexBBufferToCMSampleBuffer(const uint8* annexb_buffer, size_t annexb_buffer_size, CMVideoFormatDescriptionRef video_format, CMSampleBufferRef* out_sample_buffer, CMMemoryPoolRef memory_pool);

    // Returns a video format description created from the sps/pps information in
    // the Annex B buffer. If there is no such information, nullptr is returned.
    // The caller is responsible for releasing the description.
    static CMVideoFormatDescriptionRef CreateH264VideoFormatDescription(const uint8* annexb_buffer, size_t annexb_buffer_size);

    // Converts a sample buffer emitted from the VideoToolbox encoder into a buffer
    // suitable for RTP. The sample buffer is in avcc format whereas the rtp buffer
    // needs to be in Annex B format. Data is written directly to |annexb_buffer|.
    static bool H265CMSampleBufferToAnnexBBuffer(CMSampleBufferRef avcc_sample_buffer, bool is_keyframe, TArray<uint8>& annexb_buffer);

    // Converts a buffer received from RTP into a sample buffer suitable for the
    // VideoToolbox decoder. The RTP buffer is in annex b format whereas the sample
    // buffer is in hvcc format.
    // If |is_keyframe| is true then |video_format| is ignored since the format will
    // be read from the buffer. Otherwise |video_format| must be provided.
    // Caller is responsible for releasing the created sample buffer.
    static bool H265AnnexBBufferToCMSampleBuffer(const uint8* annexb_buffer, size_t annexb_buffer_size, CMVideoFormatDescriptionRef video_format, CMSampleBufferRef* out_sample_buffer, CMMemoryPoolRef memory_pool);

    // Returns a video format description created from the sps/pps information in
    // the Annex B buffer. If there is no such information, nullptr is returned.
    // The caller is responsible for releasing the description.
    static CMVideoFormatDescriptionRef CreateH265VideoFormatDescription(const uint8* annexb_buffer, size_t annexb_buffer_size);

    // Converts a buffer received from RTP into a sample buffer suitable for the
    // VideoToolbox decoder.
    // Caller is responsible for releasing the created sample buffer.
    static bool VP9BufferToCMSampleBuffer(const uint8* buffer, size_t buffer_size, CMVideoFormatDescriptionRef video_format, CMSampleBufferRef* out_sample_buffer, CMMemoryPoolRef memory_pool);

    static CMVideoFormatDescriptionRef CreateVP9VideoFormatDescription(const uint8* rtp_buffer, size_t rtp_buffer_size);

    // Helper class for reading NALUs from an RTP Annex B buffer.
    class AnnexBBufferReader final 
    {
    public:
        AnnexBBufferReader(const uint8* annexb_buffer, size_t length, bool isH264 = true);
        ~AnnexBBufferReader();
        AnnexBBufferReader(const AnnexBBufferReader& other) = delete;
        void operator=(const AnnexBBufferReader& other) = delete;

        // Returns a pointer to the beginning of the next NALU slice without the
        // header bytes and its length. Returns false if no more slices remain.
        bool ReadNalu(const uint8** out_nalu, size_t* out_length);

        // Returns the number of unread NALU bytes, including the size of the header.
        // If the buffer has no remaining NALUs this will return zero.
        size_t BytesRemaining() const;
        size_t BytesRemainingForAVC() const;

        // Reset the reader to start reading from the first NALU
        void SeekToStart();

        // Seek to the next position that holds a NALU of the desired type,
        // or the end if no such NALU is found.
        // Return true if a NALU of the desired type is found, false if we
        // reached the end instead
        bool SeekToNextNaluOfType(UE::AVCodecCore::H264::ENaluType type);
        bool SeekToNextNaluOfType(UE::AVCodecCore::H265::ENaluType type);

    private:
        UE::AVCodecCore::H264::ENaluType ParseH264NaluType(uint8 data);
        UE::AVCodecCore::H265::ENaluType ParseH265NaluType(uint8 data);

        const uint8* const start_;

        const size_t length_;
        
        struct NaluIndex 
        {
          // Start index of NALU, including start sequence.
          size_t start_offset;
          // Start index of NALU payload, typically type header.
          size_t payload_start_offset;
          // Length of NALU payload, in bytes, counting from payload_start_offset.
          size_t payload_size;
        };

        struct FFakeDeleter
	    {
		    void operator()(uint8* Object) const
		    {
            }
	    };
        
        TArray<NaluIndex> offsets_;
        TArray<NaluIndex>::TIterator offset_;
    };

    // Helper class for writing NALUs using avcc format into a buffer.
    class AvccBufferWriter final 
    {
    public:
        AvccBufferWriter(uint8* const avcc_buffer, size_t length);
        ~AvccBufferWriter() {}
        AvccBufferWriter(const AvccBufferWriter& other) = delete;
        void operator=(const AvccBufferWriter& other) = delete;

        // Writes the data slice into the buffer. Returns false if there isn't
        // enough space left.
        bool WriteNalu(const uint8* data, size_t data_size);

        // Returns the unused bytes in the buffer.
        size_t BytesRemaining() const;

    private:
        uint8* const start_;
        size_t offset_;
        const size_t length_;
    };
};
