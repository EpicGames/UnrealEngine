// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/UnrealString.h"

#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
#endif

THIRD_PARTY_INCLUDES_START
#include <VideoToolbox/VideoToolbox.h>
THIRD_PARTY_INCLUDES_END

class VTSessionHelpers
{
public:
    static FString CFStringToString(const CFStringRef CfString);
	static void SetVTSessionProperty(VTSessionRef Session, CFStringRef Key, int32_t Value);
	static void SetVTSessionProperty(VTSessionRef Session, CFStringRef Key, bool Value);
	static void SetVTSessionProperty(VTSessionRef Session, CFStringRef Key, CFStringRef Value);
};
