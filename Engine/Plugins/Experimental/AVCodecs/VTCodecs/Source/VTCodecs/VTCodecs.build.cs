// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;
using System;

public class VTCodecs : ModuleRules
{
    public VTCodecs(ReadOnlyTargetRules Target) : base(Target)
    {
		bRequiresPlatformSDK = true;
		
        // Without these two compilation fails on VS2017 with D8049: command line is too long to fit in debug record.
        bLegacyPublicIncludePaths = false;
		DefaultBuildSettings = BuildSettingsVersion.Latest;

        // PCHUsage = PCHUsageMode.NoPCHs;

        // PrecompileForTargets = PrecompileTargetsType.None;

        PublicIncludePaths.AddRange(new string[] {
            // ... add public include paths required here ...
        });

        PrivateIncludePaths.AddRange(new string[] {
            // ... add other private include paths required here ...
        });

        PrivateDependencyModuleNames.AddRange(new string[] {
            "Engine",
            "AVCodecsCore",
            "RHI",
            "MetalRHI"
        });

        PublicDependencyModuleNames.AddRange(new string[] {
            "RenderCore",
            "Core",
            "RHI",
        });

        PublicFrameworks.AddRange(new string[]{
            "AVFoundation",
            "VideoToolbox"
        });

        AddEngineThirdPartyPrivateStaticDependencies(Target, "MetalCPP");
    }
}
