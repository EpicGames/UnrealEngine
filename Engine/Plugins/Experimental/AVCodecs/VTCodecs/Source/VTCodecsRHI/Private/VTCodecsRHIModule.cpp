// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"
#include "Misc/App.h"
#include "Misc/CoreDelegates.h"
#include "RHI.h"

#include "VT.h"
#include "AVUtility.h"


class FVTCodecRHI : public IModuleInterface
{
public:
    virtual void StartupModule() override
    {
        if (FApp::CanEverRender())
        {
            FCoreDelegates::GetOnPostEngineInit().AddLambda([]()
                {
#if PLATFORM_IOS
					// On iOS, GRHIVendorId is not set so IsRHIDeviceApple returns false
					const_cast<FVT&>(FAPI::Get<FVT>()).bHasCompatibleGPU = true;
#else // PLATFORM_IOS
					const_cast<FVT&>(FAPI::Get<FVT>()).bHasCompatibleGPU = IsRHIDeviceApple();
#endif // PLATFORM_IOS
                });
        }
    }
};

IMPLEMENT_MODULE(FVTCodecRHI, VTCodecsRHI);
