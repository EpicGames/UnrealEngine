// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;
using System;

public class VTCodecsRHI : ModuleRules
{
    public VTCodecsRHI(ReadOnlyTargetRules Target) : base(Target)
    {
		bRequiresPlatformSDK = true;
        bLegacyPublicIncludePaths = false;
		DefaultBuildSettings = BuildSettingsVersion.Latest;
        
        PrivateDependencyModuleNames.AddRange(new string[] {
            "Core",
            "Engine",
            "AVCodecsCore",
            "RHI",
            "VTCodecs"
        });

        PublicDependencyModuleNames.AddRange(new string[] {
            "RenderCore",
            "Core",
            "CoreUObject",
        });
        
        PublicFrameworks.AddRange(new string[]{
            "AVFoundation",
            "VideoToolbox"
        });
    }
}
