{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "VTCodecs",
	"Description": "Adds codecs from the Apple Video Toolbox Framework to AVCodecs",
	"Category": "Codecs",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "http://epicgames.com",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "VTCodecs",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit",
			"TargetDenyList": [
				"Server"
			],
			"PlatformAllowList": [
				"Mac",
				"IOS"
			]
		},
		{
			"Name": "VTCodecsRHI",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit",
			"TargetDenyList": [
				"Server"
			],
			"PlatformAllowList": [
				"Mac",
				"IOS"
			]
		}
	],
	"Plugins": [
		{
			"Name": "AVCodecsCore",
			"Enabled": true
		}
	],
	"SupportedTargetPlatforms": [
		"Mac",
		"IOS"
	]
}